{
	"ImporterCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows/torch-mlir-import-onnx.exe" ],
	"ImporterArguments": "-o ${OUTPUT_PATH} --opset-version 21 ${INPUT_PATH}",
	"CompilerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows/iree-compile.exe" ],
	"Targets": [
		{
			"ShaderPlatform": "PCD3D_SM6",
			"CompilerArguments": "--iree-hal-target-backends=unreal-shader --iree-unreal-target=sm_75 --iree-hal-dump-executable-binaries-to=${BINARIES_PATH} -o ${VMFB_PATH} ${INPUT_PATH}"
		}
	]
}