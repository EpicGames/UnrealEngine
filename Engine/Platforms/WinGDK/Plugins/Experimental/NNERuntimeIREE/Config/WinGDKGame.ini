[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysStageAsNonUFS=(Path="../Binaries/WinGDK/NNERuntimeIREECpu")
