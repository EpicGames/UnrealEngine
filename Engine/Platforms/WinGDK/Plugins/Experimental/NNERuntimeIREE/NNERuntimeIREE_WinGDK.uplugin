{
	"FileVersion": 3,
	"bIsPluginExtension": true
}