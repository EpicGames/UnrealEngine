@ECHO off
SETLOCAL

if "%~3" == "" (
	echo At least 3 arguments are required: [Build directory] [Build type] [CMake module path]
	GOTO :fail
)

SET "BUILD_DIR=%1"
SET "BUILD_TYPE=%2"
SET "CMAKE_MODULE_PATH=%3"

SET "PLATFORM_BUILD_DIR=%BUILD_DIR%\iree-runtime-WinGDK"

SET "BUILD_SCRIPT_DIR=%~dp0"
SET "UE_SOURCE_DIR=%BUILD_SCRIPT_DIR%\..\..\Source"

ECHO Building in "%BUILD_DIR%"
ECHO From source in "%UE_SOURCE_DIR%"

ECHO =========================================
ECHO ======== Building WinGDK Runtime ========
ECHO =========================================

IF NOT EXIST "%PLATFORM_BUILD_DIR%" (
	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_RUNTIME% -vcvars_ver=%VC_TOOLSET%
	cmake -G Ninja -B "%PLATFORM_BUILD_DIR%" "%UE_SOURCE_DIR%\iree\runtime" ^
		-DCMAKE_MODULE_PATH="%CMAKE_MODULE_PATH%" ^
		-DCMAKE_BUILD_TYPE=%BUILD_TYPE% ^
		-DUE_IREE_BUILD_ROOT="%BUILD_DIR%" ^
		-DIREE_ENABLE_CPUINFO=OFF ^
		-DIREE_BUILD_TESTS=OFF ^
		-DIREE_BUILD_SAMPLES=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF ^
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF ^
		-DIREE_HAL_DRIVER_DEFAULTS=OFF ^
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON ^
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON ^
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF ^
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON ^
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF ^
		-DIREE_BUILD_COMPILER=OFF ^
		-DIREE_ENABLE_THREADING=ON ^
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF ^
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	cmake --build "%PLATFORM_BUILD_DIR%"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
	ECHO Building WinGDK Runtime: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Building WinGDK Runtime: Incremental build... Please do not use for Release build.
	ECHO:

	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_RUNTIME% -vcvars_ver=%VC_TOOLSET%

	cmake --build "%PLATFORM_BUILD_DIR%"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
)

ENDLOCAL
EXIT /B 0

:fail
ECHO.
ECHO Build script failed. Exiting 1>&2
ENDLOCAL & EXIT /B 1