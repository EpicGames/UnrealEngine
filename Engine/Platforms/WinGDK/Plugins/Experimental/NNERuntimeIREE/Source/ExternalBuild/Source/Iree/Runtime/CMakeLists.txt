cmake_minimum_required(VERSION 3.21...3.24)
project(IREE_RUNTIME_XSX)

# Prevent an issue of not correctly included modules
include(CheckCSourceCompiles)

# Include main module
include(iree_runtime)