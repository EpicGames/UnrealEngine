// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IREEDriverRDG_WinGDK : IREEDriverRDG
{
	public IREEDriverRDG_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
		if (Target.Platform == UnrealTargetPlatform.WinGDK)
		{
			PublicDefinitions.Add("WITH_IREE_DRIVER_RDG");
			PrivateDefinitions.Add("IREE_DRIVER_RDG_VERBOSITY=0");
		}
	}
}
