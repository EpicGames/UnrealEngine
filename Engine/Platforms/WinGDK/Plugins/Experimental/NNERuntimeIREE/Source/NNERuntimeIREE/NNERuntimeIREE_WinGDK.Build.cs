// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeIREE_WinGDK : NNERuntimeIREE
{
	public NNERuntimeIREE_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
		if (Target.Platform == UnrealTargetPlatform.WinGDK)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE");
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_RDG");
		}
	}
}
