// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeIREEShader_WinGDK : NNERuntimeIREEShader
{
	public NNERuntimeIREEShader_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
		if (Target.Platform == UnrealTargetPlatform.WinGDK)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_SHADER");
		}
	}
}
