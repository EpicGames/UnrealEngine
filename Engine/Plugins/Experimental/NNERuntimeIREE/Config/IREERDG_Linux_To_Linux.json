{
	"ImporterCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Linux/torch-mlir-import-onnx" ],
	"ImporterArguments": "-o ${OUTPUT_PATH} --opset-version 21 ${INPUT_PATH}",
	"CompilerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Linux/iree-compile" ],
	"Targets": [
		{
			"ShaderPlatform": "VULKAN_SM6",
			"CompilerArguments": "--iree-hal-target-backends=unreal-shader --iree-unreal-target=sm_75 --iree-unreal-use-structured-buffers --iree-hal-dump-executable-binaries-to=${BINARIES_PATH} -o ${VMFB_PATH} ${INPUT_PATH}"
		}
	]
}