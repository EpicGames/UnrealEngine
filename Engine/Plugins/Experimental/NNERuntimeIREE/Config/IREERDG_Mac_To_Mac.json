{
	"ImporterCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Mac/torch-mlir-import-onnx" ],
	"ImporterArguments": "-o ${OUTPUT_PATH} --opset-version 21 ${INPUT_PATH}",
	"CompilerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Mac/iree-compile" ],
	"Targets": [
		{
			"ShaderPlatform": "METAL_SM6",
			"CompilerArguments": "--iree-hal-target-backends=unreal-shader --iree-unreal-target=apple --iree-llvmcpu-embedded-linker-path=./Binaries/ThirdParty/IREE/Mac/ld.lld --iree-hal-dump-executable-binaries-to=${BINARIES_PATH} -o ${VMFB_PATH} ${INPUT_PATH}"
		}
	]
}
