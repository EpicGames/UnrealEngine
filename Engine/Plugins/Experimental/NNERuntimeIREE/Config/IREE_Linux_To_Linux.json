{
	"ImporterCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Linux/torch-mlir-import-onnx" ],
	"ImporterArguments": "-o ${OUTPUT_PATH} --opset-version 21 ${INPUT_PATH}",
	"CompilerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Linux/iree-compile" ],
	"LinkerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Linux/ld.lld" ],
	"SharedLibExt": ".so",
	"Targets": [
		{
			"Name": "AVX512",
			"Architecture": "",
			"x86Features": [ "avx512", "fma", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-unknown-linux-gnu --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c,+avx2,+fma,+avx512f,+avx512vl,+avx512bw,+avx512dq --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-shared -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		},
		{
			"Name": "AVX2",
			"Architecture": "",
			"x86Features": [ "avx2", "fma", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-unknown-linux-gnu --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c,+avx2,+fma --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-shared -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		},
		{
			"Name": "AVX",
			"Architecture": "",
			"x86Features": [ "avx", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-unknown-linux-gnu --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-shared -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		},
		{
			"Name": "Default",
			"Architecture": "",
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-unknown-linux-gnu --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2 --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-shared -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		}
	]
}
