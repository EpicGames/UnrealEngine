{
	"ImporterCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Mac/torch-mlir-import-onnx" ],
	"ImporterArguments": "-o ${OUTPUT_PATH} --opset-version 21 ${INPUT_PATH}",
	"CompilerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Mac/iree-compile" ],
	"LinkerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Mac/ld64.lld" ],
	"SharedLibExt": ".dylib",
	"Targets": [
		{
			"Name":  "ARM",
			"Architecture": "arm64",
			"CompilerArguments": "--iree-llvmcpu-target-triple=arm64-apple-darwin --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --iree-llvmcpu-embedded-linker-path=./Binaries/ThirdParty/IREE/Mac/ld.lld --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-dylib -platform_version macos 11.0.0 11.0.0 -arch arm64 -syslibroot \"${MAC_SDK_ROOT}\" -lSystem -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		},
		{
			"Name": "Intel_AVX2",
			"Architecture": "x86_64",
			"x86Features": [ "avx2", "fma", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-apple-darwin --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c,+avx2,+fma --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --iree-llvmcpu-embedded-linker-path=./Binaries/ThirdParty/IREE/Mac/ld.lld --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-dylib -platform_version macos 11.0.0 11.0.0 -arch x86_64 -syslibroot \"${MAC_SDK_ROOT}\" -lSystem -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		},
		{
			"Name": "Intel_Default",
			"Architecture": "x86_64",
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-apple-darwin --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --iree-llvmcpu-embedded-linker-path=./Binaries/ThirdParty/IREE/Mac/ld.lld --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-dylib -platform_version macos 11.0.0 11.0.0 -arch x86_64 -syslibroot \"${MAC_SDK_ROOT}\" -lSystem -o ${SHARED_LIB_PATH} ${OBJECT_PATH}"
		}
	]
}