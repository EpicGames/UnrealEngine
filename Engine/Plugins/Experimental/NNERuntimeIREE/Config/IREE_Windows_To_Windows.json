{
	"ImporterCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows/torch-mlir-import-onnx.exe" ],
	"ImporterArguments": "-o ${OUTPUT_PATH} --opset-version 21 ${INPUT_PATH}",
	"CompilerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows/iree-compile.exe" ],
	"LinkerCommand": [ "${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows/lld-link.exe" ],
	"SharedLibExt": ".dll",
	"Targets": [
		{
			"Name": "AVX512",
			"Architecture": "",
			"x86Features": [ "avx512", "fma", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-pc-windows-msvc --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c,+avx2,+fma,+avx512f,+avx512vl,+avx512bw,+avx512dq --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-out:${SHARED_LIB_PATH} -nodefaultlib -libpath:${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows clang_rt.builtins-x86_64.lib ucrtbase_applocal.lib libfltused.lib -noentry -nologo -dll -alternatename:__chkstk=___chkstk_ms ${OBJECT_PATH}"
		},
		{
			"Name": "AVX2",
			"Architecture": "",
			"x86Features": [ "avx2", "fma", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-pc-windows-msvc --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c,+avx2,+fma --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-out:${SHARED_LIB_PATH} -nodefaultlib -libpath:${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows clang_rt.builtins-x86_64.lib ucrtbase_applocal.lib libfltused.lib -noentry -nologo -dll -alternatename:__chkstk=___chkstk_ms ${OBJECT_PATH}"
		},
		{
			"Name": "AVX",
			"Architecture": "",
			"x86Features": [ "avx", "f16c" ],
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-pc-windows-msvc --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2,+avx,+f16c --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-out:${SHARED_LIB_PATH} -nodefaultlib -libpath:${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows clang_rt.builtins-x86_64.lib ucrtbase_applocal.lib libfltused.lib -noentry -nologo -dll -alternatename:__chkstk=___chkstk_ms ${OBJECT_PATH}"
		},
		{
			"Name": "Default",
			"Architecture": "",
			"CompilerArguments": "--iree-llvmcpu-target-triple=x86_64-pc-windows-msvc --iree-input-type=auto --iree-hal-target-backends=llvm-cpu --iree-llvmcpu-target-cpu-features=+sse,+sse2,+ssse3,+sse4.1,+sse4.2 --iree-llvmcpu-link-embedded=false --iree-llvmcpu-link-static --output-format=vm-bytecode --iree-llvmcpu-static-library-output-path=${OBJECT_PATH} -o ${VMFB_PATH} ${INPUT_PATH}",
			"LinkerArguments": "-out:${SHARED_LIB_PATH} -nodefaultlib -libpath:${PLUGIN_DIR}/Binaries/ThirdParty/IREE/Windows clang_rt.builtins-x86_64.lib ucrtbase_applocal.lib libfltused.lib -noentry -nologo -dll -alternatename:__chkstk=___chkstk_ms ${OBJECT_PATH}"
		}
	]
}