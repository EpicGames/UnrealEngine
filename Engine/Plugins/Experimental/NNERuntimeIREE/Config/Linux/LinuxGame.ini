[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysStageAsNonUFS=(Path="../Binaries/Linux/NNERuntimeIREECpu")
