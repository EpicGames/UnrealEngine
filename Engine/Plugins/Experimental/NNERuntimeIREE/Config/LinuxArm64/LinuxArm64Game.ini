[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysStageAsNonUFS=(Path="../Binaries/LinuxArm64/NNERuntimeIREECpu")
