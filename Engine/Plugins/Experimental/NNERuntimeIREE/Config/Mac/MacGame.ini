[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysStageAsNonUFS=(Path="../Binaries/Mac/NNERuntimeIREECpu")
