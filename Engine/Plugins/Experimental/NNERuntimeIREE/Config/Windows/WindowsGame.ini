[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysStageAsNonUFS=(Path="../Binaries/Win64/NNERuntimeIREECpu")
