{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "NNERuntimeIREE",
	"Description": "A runtime implementing the Neural Network Engine (NNE) API which is based on IREE, MLIR and LLVM and compiles neural networks directly to game code.",
	"Category": "ML",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "IREETracing",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "IREEUtils",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "IREEDriverRDG",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "NNERuntimeIREE",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "NNERuntimeIREEEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "NNERuntimeIREEShader",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		}
	]
}