// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

RWByteAddressBuffer TargetBuffer;

// Fill parameters
// x: pattern
// y: pattern width
// z: fill offset in bytes
// w: fill length in bytes
uint4 Fill;

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void Main(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Address = Fill.z + DispatchThreadID.x * 4;
	if (Address < Fill.z + Fill.w)
	{
		TargetBuffer.Store(Address, Fill.x);
	}
}