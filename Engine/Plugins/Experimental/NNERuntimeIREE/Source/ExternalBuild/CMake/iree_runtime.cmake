cmake_minimum_required(VERSION 3.21...3.24)

if (NOT UE_IREE_BUILD_ROOT)
	message(FATAL_ERROR "Error: UE_IREE_BUILD_ROOT not set")
endif()

if (NOT UE_IREE_ADDITIONAL_SOURCES)
	set(UE_IREE_ADDITIONAL_SOURCES "")
endif()

if (NOT UE_IREE_RUNTIME_LIBRARY_NAME)
	set(UE_IREE_RUNTIME_LIBRARY_NAME "ireert")
endif()

# Set device standalone to make sure some functions are included to avoid mismatch on target systems (e.g. __truncsfhf2 and __extendhfsf2)
add_compile_definitions(IREE_DEVICE_STANDALONE)

add_subdirectory("${UE_IREE_BUILD_ROOT}/iree" "iree" EXCLUDE_FROM_ALL)

# Get all targets required by the runtime
set(_RUNTIME_ROOT "iree::runtime")
set(_RUNTIME_OBJECTS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_RUNTIME_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECTS>>>")
set(_RUNTIME_LIBS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_RUNTIME_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECT_LIBS>>>")

set(_LOADER_ROOT "iree::hal::local::loaders::static_library_loader")
set(_LOADER_OBJECTS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_LOADER_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECTS>>>")
set(_LOADER_LIBS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_LOADER_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECT_LIBS>>>")

set(_ELF_ROOT "iree::hal::local::elf::arch")
set(_ELF_OBJECTS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_ELF_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECTS>>>")
set(_ELF_LIBS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_ELF_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECT_LIBS>>>")

set(_DEVICE_ROOT "iree::builtins::device::device")
set(_DEVICE_OBJECTS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_DEVICE_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECTS>>>")
set(_DEVICE_LIBS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_DEVICE_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECT_LIBS>>>")

# HAL utilities required by the custom RDG driver (not transitively pulled by iree::runtime)
set(_EXEC_HEADER_ROOT "iree::hal::utils::executable_header")
set(_EXEC_HEADER_OBJECTS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_EXEC_HEADER_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECTS>>>")
set(_EXEC_HEADER_LIBS "$<REMOVE_DUPLICATES:$<GENEX_EVAL:$<TARGET_PROPERTY:${_EXEC_HEADER_ROOT},INTERFACE_IREE_TRANSITIVE_OBJECT_LIBS>>>")

# eyalroz/printf library used by iree::base for platform-independent printf
set(_PRINTF_OBJECTS "$<TARGET_OBJECTS:printf>")

# Define the library
add_library(${UE_IREE_RUNTIME_LIBRARY_NAME} ${_RUNTIME_OBJECTS} ${_LOADER_OBJECTS} ${_ELF_OBJECTS} ${_DEVICE_OBJECTS} ${_EXEC_HEADER_OBJECTS} ${_PRINTF_OBJECTS} ${UE_IREE_ADDITIONAL_SOURCES})
target_link_libraries(${UE_IREE_RUNTIME_LIBRARY_NAME} PUBLIC ${_RUNTIME_LIBS} ${_LOADER_LIBS} ${_ELF_LIBS} ${_DEVICE_LIBS} ${_EXEC_HEADER_LIBS})