// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef __cplusplus
extern "C" {
#endif // __cplusplus

void *alloca(size_t);

#define alloca __builtin_alloca

#ifdef __cplusplus
}
#endif // __cplusplus