diff --git a/build_tools/cmake/flatbuffer_c_library.cmake b/build_tools/cmake/flatbuffer_c_library.cmake
index 85c6f4cd5f..89e9d8b1e1 100644
--- a/build_tools/cmake/flatbuffer_c_library.cmake
+++ b/build_tools/cmake/flatbuffer_c_library.cmake
@@ -88,7 +88,11 @@ function(flatbuffer_c_library)
   # When cross-compiling execute the binary from the host build.
   set(IREE_FLATCC_TOOL_BINARY iree-flatcc-cli)
   if (IREE_HOST_BIN_DIR)
-    set(IREE_FLATCC_TOOL_BINARY "${IREE_HOST_BIN_DIR}/iree-flatcc-cli")
+    set(_HOST_EXECUTABLE_SUFFIX "")
+    if(CMAKE_HOST_WIN32)
+      set(_HOST_EXECUTABLE_SUFFIX ".exe")
+    endif()
+    set(IREE_FLATCC_TOOL_BINARY "${IREE_HOST_BIN_DIR}/iree-flatcc-cli${_HOST_EXECUTABLE_SUFFIX}")
   endif()
 
   add_custom_command(
