diff --git a/runtime/src/iree/builtins/device/device_generic.c b/runtime/src/iree/builtins/device/device_generic.c
index 000000000000..111111111111 100644
--- a/runtime/src/iree/builtins/device/device_generic.c
+++ b/runtime/src/iree/builtins/device/device_generic.c
@@ -169,8 +169,10 @@ IREE_DEVICE_EXPORT float __truncdfhf2(double param) {
   return __truncsfhf2((float)param);
 }
 
+#ifndef _MSC_VER
 IREE_DEVICE_EXPORT double fma(double x, double y, double z) {
   return x * y + z;
 }
+#endif
 
 #endif  // IREE_DEVICE_STANDALONE
