diff --git a/compiler-rt/lib/builtins/CMakeLists.txt b/compiler-rt/lib/builtins/CMakeLists.txt
index 075c4647abf6..f53feceb870c 100644
--- a/compiler-rt/lib/builtins/CMakeLists.txt
+++ b/compiler-rt/lib/builtins/CMakeLists.txt
@@ -398,6 +398,7 @@ else () # MSVC
     ${x86_ARCH_SOURCES}
     x86_64/floatdidf.c
     x86_64/floatdisf.c
+    x86_64/chkstk.S
   )
   set(i386_SOURCES ${GENERIC_SOURCES} ${x86_ARCH_SOURCES})
 endif () # if (NOT MSVC)
