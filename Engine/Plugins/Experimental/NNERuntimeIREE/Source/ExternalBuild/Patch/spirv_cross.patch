diff --git a/spirv_hlsl.cpp b/spirv_hlsl.cpp
index 9b834936..b04e7a38 100644
--- a/spirv_hlsl.cpp
+++ b/spirv_hlsl.cpp
@@ -491,9 +492,13 @@ string CompilerHLSL::type_to_glsl(const SPIRType &type, uint32_t id)
 		case SPIRType::Double:
 			return join("double", type.vecsize);
 		case SPIRType::Int64:
-			return join("i64vec", type.vecsize);
+			if (hlsl_options.shader_model < 60)
+				SPIRV_CROSS_THROW("64-bit integers only supported in SM 6.0.");
+			return join("int64_t", type.vecsize);
 		case SPIRType::UInt64:
-			return join("u64vec", type.vecsize);
+			if (hlsl_options.shader_model < 60)
+				SPIRV_CROSS_THROW("64-bit integers only supported in SM 6.0.");
+			return join("uint64_t", type.vecsize);
 		default:
 			return "???";
 		}
@@ -1966,6 +1971,29 @@ void CompilerHLSL::emit_resources()
 		statement("");
 	}
 
+	if (UE_requires_half4_packing || UE_requires_float2_unpacking)
+	{
+		// HLSL does not pack into a single word sadly :(
+		statement("uint2 spvPackHalf4(half4 value)");
+		begin_scope();
+		statement("uint4 Packed = f32tof16(value);");
+		statement("return uint2((0x0000ffff & Packed.x) | (Packed.y << 16), (0x0000ffff & Packed.z) | (Packed.w << 16));");
+		end_scope();
+		statement("");
+
+		statement("half4 spvUnpackHalf4(uint2 value)");
+		begin_scope();
+		statement("return half4(f16tof32(uint4(value.x & 0xffff, value.x >> 16, value.y & 0xffff, value.y >> 16)));");
+		end_scope();
+		statement("");
+
+		statement("half4 spvUnpackFloat2(float2 value)");
+		begin_scope();
+		statement("return spvUnpackHalf4(asuint(value));");
+		end_scope();
+		statement("");
+	}
+
 	// HLSL does not seem to have builtins for these operation, so roll them by hand ...
 	if (requires_unorm8_packing)
 	{
@@ -4190,6 +4218,33 @@ string CompilerHLSL::bitcast_glsl_op(const SPIRType &out_type, const SPIRType &i
 			SPIRV_CROSS_THROW("UShort to Half requires Shader Model 4.");
 		return "(" + type_to_glsl(out_type) + ")f16tof32";
 	}
+	else if (out_type.basetype == SPIRType::UInt && out_type.vecsize == 2 && in_type.basetype == SPIRType::Half && in_type.vecsize == 4)
+	{
+		if (!UE_requires_half4_packing)
+		{
+			UE_requires_half4_packing = true;
+			force_recompile();
+		}
+		return "spvPackHalf4";
+	}
+	else if (out_type.basetype == SPIRType::Half && out_type.vecsize == 4 && in_type.basetype == SPIRType::UInt && in_type.vecsize == 2)
+	{
+		if (!UE_requires_half4_packing)
+		{
+			UE_requires_half4_packing = true;
+			force_recompile();
+		}
+		return "spvUnpackHalf4";
+	}
+	else if (out_type.basetype == SPIRType::Half && out_type.vecsize == 4 && in_type.basetype == SPIRType::Float && in_type.vecsize == 2)
+	{
+		if (!UE_requires_float2_unpacking)
+		{
+			UE_requires_float2_unpacking = true;
+			force_recompile();
+		}
+		return "spvUnpackFloat2";
+	}
 	else
 		return "";
 }
diff --git a/spirv_hlsl.hpp b/spirv_hlsl.hpp
index bec458c6..7f90960f 100644
--- a/spirv_hlsl.hpp
+++ b/spirv_hlsl.hpp
@@ -314,6 +314,9 @@ private:
 	bool requires_scalar_refract = false;
 	bool requires_scalar_faceforward = false;
 
+	bool UE_requires_half4_packing = false;
+	bool UE_requires_float2_unpacking = false;
+
 	struct TextureSizeVariants
 	{
 		// MSVC 2013 workaround.
