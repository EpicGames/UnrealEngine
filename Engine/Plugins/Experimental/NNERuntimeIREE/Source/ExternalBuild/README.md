# NNERuntimeIREE build external dependencies
Scripts to clone and build the IREE compiler as well as the runtime libraries. Additionally will build the shared library NNEMlirTools that is used to parse and inspect MLIR. The library is based on the same IREE clone to enable maximum compability.

## Windows

### Setup & Requirements

- Install Visual Studio 2022
- Start the Visual Studio Installer, click 'Modify' and change to the 'Individual Components' tab
- Check which toolchains are supported in UE (written in Engine\Config\Windows\Windows_SDK.json in PreferredVisualCppVersions, e.g "14.38.33130-14.38.99999" // VS2022 17.8.x for UE 5.5 and write down the lowest supported MSVC-VS versions, futher down in this doc it will be called *lowest MSVC-VS XX.XX-XX.X* or *lowest VS XX.X*. For UE 5.5 it is thus 14.38-17.8.
- Check which SDK is supported in UE (also written in Engine\Config\Windows\Windows_SDK.json but under "MinVersion") it will be called *lowest supported WINSDK ver* in this doc.
- Make sure you install the following toolchains (and only the following, there were some issues when mixing different toolchains)
  - C++ Modules for v143 build tools (x64/x86 - experimental)
  - C++/CLI support for v143 ( *lowest MSVC-VS XX.XX-XX.X*)
  - MSBuild
  - MSBuild support for LLVM (clang-cl) toolset
  - MSVC v143 - VS 2022 C++ ARM build tools (*lowest MSVC-VS XX.XX-XX.X*)
  - MSVC v143 - VS 2022 C++ ARM Spectre-mitigated libs (*lowest MSVC-VS XX.XX-XX.X*)
  - MSVC v143 - VS 2022 C++ ARM64/ARM64EC build tools (*lowest MSVC-VS XX.XX-XX.X*)
  - MSVC v143 - VS 2022 C++ ARM64/ARM64EC Spectre-mitigated libs (*lowest MSVC-VS XX.XX-XX.X*)
  - MSVC v143 - VS 2022 C++ x64/x86 build tools (*lowest MSVC-VS XX.XX-XX.X*)
  - MSVC v143 - VS 2022 C++ x64/x86 Spectre-mitigated libs (*lowest MSVC-VS XX.XX-XX.X*)
  - MSVC v143 C++ X64/x86 build tools (Latest) <-- required to get 'vcvarsall.bat'
- Also install
  - All six C++ ATL for (*lowest VS XX.X*) v143 build tools
  - All six C++ MFC for (*lowest VS XX.X*) v143 build tools
  - Windows SDK (*lowest supported WINSDK ver*) witch is used for the runtime lib
  - Windows 11 SDK (10.0.22621.0) witch is used for the iree compiler
- Install Cmake and also ninja as described on the IREE webpage
  - In addition, set the system environment variable NINJA_PATH to your executable (e.g. D:/ninja/ninja.exe)
  - Set the vcvarsall.bat path to an environment variable called VCVARSALL_PATH (e.g.C:\Program Files\Microsoft Visual Studio\2022\Professional\VC\Auxiliary\Build\vcvarsall.bat for UE 5.5)
- Install all console SDKs and make sure the corresponding environment variables are set
- Install git command tools

## Building & Install

The build scripts can be found in the subdirectory 'Scripts'. For every build platform there is a subdirectory:
- `Build*.*`: Main build script.
- `Install*.*`: Script that copies artifacts to their specific location within the NNERuntimeIREE plugin and finally creates a CL in Perforce with the changes included.
- `BuildAndInstall*.*`: Convencience script that calls `Build*.*` and `Install*.*`.

For customization you can pass the intermediate build directory path to the scripts. On Windows you also can provide the additional platforms the IREE runtime should be built for (only if supported). Or by specifing 'All', the script will look by itself for all platforms that are supported and build the runtime for them.

*Windows*

`Script.bat [-dir WorkingDir] [-platform "Platform1 Platform2 ..."]`

*Linux & Mac*

`Script.sh [WorkingDir]`

**Important note**

Since source paths can be shown in IREE error messages, be adviced for production builds to carefully setup the folder structure. A common approach we choose is to build in `github.com` which is located at the root of the preferred volume or disk (e.g. `E:\github.com` on Windows, `/github.com` on Linux, or `/Volumes/MyVolume/github.com` on Mac).

## Linux

- Install cmake and ninja-build
- Ensure v23_clang-18.1.0-rockylinux8.tar.gz is still the compiler used for UE (was at 5.5)
  * To do so you need to get UE building and launch any sample project. Then navigate to /Engine/Build/BatchFiles/Linux/ and execute ./GenerateProjectFiles.sh -cmakefile -game "/path/to/the/sample/project.uproject" you will see in the logs what is used compiler. If different you need to adapt COMPILER_NAME in BuildLinux.sh
- The llvm project is built with clang-16, so make sure this compiler is installed too
- clang-16 requires libstdc++-12, it can be installed from the default repositories: sudo apt install libstdc++-12-dev.
 
Note: atm llvm cannot be compiled using the clang that UE is using. Thus a different compiler is used for this.
