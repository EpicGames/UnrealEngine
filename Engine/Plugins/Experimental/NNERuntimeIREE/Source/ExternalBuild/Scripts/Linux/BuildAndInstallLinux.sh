#! /bin/bash

#
# Script that calls build and install scripts together with the same arguments.
#
# If any intermediate step fails, the script (should) exit.
#

set -Eeuo pipefail

trap 'echo; echo "Script failed. Exiting" >&2; exit 1' ERR

SCRIPT_DIR="$(cd -- "$(dirname -- "${BASH_SOURCE[0]}")" &>/dev/null && pwd)"

# For production be aware that the build path is part of the IREE error messages
# So choose it with care
DEFAULT_WORKING_DIR="$SCRIPT_DIR/../../Build"

if [ -z "${1-}" ]; then
	mkdir -p "$DEFAULT_WORKING_DIR"
	WORKING_DIR="$DEFAULT_WORKING_DIR"
else
	WORKING_DIR="$1"
	if [ ! -d "$WORKING_DIR" ]; then
		echo "Error: '$WORKING_DIR' does not exist." >&2
		exit 1
	fi
fi

WORKING_DIR="$(cd "$WORKING_DIR" && pwd)"

echo
echo "Working dir: \"$WORKING_DIR\""
echo

"${SCRIPT_DIR}/BuildLinux.sh" "$WORKING_DIR"
"${SCRIPT_DIR}/InstallLinux.sh" "$WORKING_DIR"

exit 0