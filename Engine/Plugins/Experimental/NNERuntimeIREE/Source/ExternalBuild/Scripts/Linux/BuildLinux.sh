#! /bin/bash

#
# This script builds all the required targets for the NNERuntimeIREE plugin.
#
# If an intermediate step fails, the script exits.
#

set -euo pipefail

if [ -z "$1" ]; then
	echo "Usage: [Script] [Working dir]"
	exit 1
fi

WORKING_DIR=$1

BUILD_DIR=$WORKING_DIR/iree-org
BUILD_SCRIPT_DIR="$(dirname -- "$(realpath -- "$0")")/"
UE_SOURCE_DIR=$BUILD_SCRIPT_DIR/../../Source
UE_PATCHES_DIR=$BUILD_SCRIPT_DIR/../../Patch
UE_CMAKE_DIR=$BUILD_SCRIPT_DIR/../../CMake
UE_PLUGIN_ROOT=$BUILD_SCRIPT_DIR/../../../..

UE_PLUGIN_ROOT="$(realpath -e -- "$UE_PLUGIN_ROOT")"
UE_SOURCE_DIR="$(realpath -e -- "$UE_SOURCE_DIR")"

echo Plugin root dir: "$UE_PLUGIN_ROOT"
echo Build dir: "$BUILD_DIR"
echo Source dir: "$UE_SOURCE_DIR"

IREE_GIT_REPOSITORY=https://github.com/iree-org/iree.git
IREE_GIT_COMMIT=v3.11.0
IREE_THIRD_PARTY_LIBRARIES=("flatcc" "llvm-project" "stablehlo" "torch-mlir" "benchmark" "spirv_cross" "printf")
IREE_COMPILER_VERSION_STRING="IREE-for-UE"

BUILD_TYPE=Release
DEV_BUILD_TYPE=RelWithDebInfo

# Unreal Shader compiler plugin
IREE_CMAKE_PLUGIN_PATH=../../Iree/Compiler/Plugins/Target/UnrealShader

SPIRV_CROSS_DIR=$BUILD_DIR/iree/third_party/spirv_cross

# Change this when compiling on arch64 for arch64
TARGET=x86_64-unknown-linux-gnu #TARGET=aarch64-unknown-linux-gnueabi

COMPILER_NAME=v26_clang-20.1.8-rockylinux8
C_COMPILER=$BUILD_DIR/Clang/$COMPILER_NAME/$TARGET/bin/clang
CXX_COMPILER=$BUILD_DIR/Clang/$COMPILER_NAME/$TARGET/bin/clang++
COMPILER_FLAGS=--target=$TARGET

# Common rsync options
# note: -L to copy "resolved" symlinks (e.g. ld.lld)
RSYNC_OPTIONS=(-a -L)
sync() { rsync "${RSYNC_OPTIONS[@]}" -- "$@"; }

if [ ! -d $BUILD_DIR ] ; then
	mkdir $BUILD_DIR
fi

echo =========================================
echo ========= INSTALLING PATCHELF ===========
echo =========================================

sudo apt-get install patchelf

echo =========================================
echo =========== INSTALLING CLANG ============
echo =========================================

if [ ! -f "$BUILD_DIR/Clang/native-linux-$COMPILER_NAME.tar.gz" ]; then
	cd $BUILD_DIR

	if [ ! -d Clang ] ; then
		mkdir Clang
	fi

	cd Clang

	if [ ! -f native-linux-$COMPILER_NAME.tar.gz ] ; then
		wget https://cdn.unrealengine.com/Toolchain_Linux/native-linux-$COMPILER_NAME.tar.gz
	fi
	if [ ! -d v$COMPILER_NAME ] ; then
		tar -xzf native-linux-$COMPILER_NAME.tar.gz
	fi

	if [ ! -f llvm.sh ] ; then
		wget https://apt.llvm.org/llvm.sh
		chmod 777 llvm.sh
	fi
	sudo ./llvm.sh 20 all

	cd ../..
	echo
	echo Installing Clang: Done
	echo
else
	echo
	echo Installing Clang: Skipped
	echo
fi


echo =========================================
echo ============= Copy Source ===============
echo =========================================

mkdir -p "$WORKING_DIR/Iree/Compiler"
sync "$UE_SOURCE_DIR/Iree/Compiler/" "$WORKING_DIR/Iree/Compiler"

echo
echo Copied IREE compiler plugin.
echo


echo =========================================
echo ============= Cloning IREE ==============
echo =========================================

if [ ! -d "$BUILD_DIR/iree" ]; then
	cd $BUILD_DIR
	git clone -n $IREE_GIT_REPOSITORY
	cd iree
	echo Using IREE git commit $IREE_GIT_COMMIT
	git checkout $IREE_GIT_COMMIT
	cd third_party
	for D in */; do
		if [ ! -f "$D/CMakeLists.txt" ]; then
			echo -n "" > "$D/CMakeLists.txt"
		fi
	done
	for L in ${IREE_THIRD_PARTY_LIBRARIES[@]}; do
		if [ -d $L ]; then
			rm "$L/CMakeLists.txt"
			git submodule update --init -- $L
			if [ "$L" != "llvm-project" ] && [ -d "$L/third-party" ]; then
				rm -R -f "$L/third-party"
			fi
		fi
	done

	cd ../../..

	echo Check for $SPIRV_CROSS_DIR
	if [ -d "$SPIRV_CROSS_DIR" ]; then
		echo Apply git patch to spirv_cross
		cd $SPIRV_CROSS_DIR
		git apply "$UE_PATCHES_DIR/spirv_cross.patch"
		cd ../../../..
		echo Done.
	fi

	echo
	echo Cloning IREE: Done
	echo
else
	echo
	echo Cloning IREE: Skipped
	echo
fi

echo =========================================
echo ======== Building IREE Compiler =========
echo =========================================

if [ ! -d "$BUILD_DIR/iree-compiler" ]; then
	cmake -G Ninja -B $BUILD_DIR/iree-compiler $BUILD_DIR/iree \
		-DCMAKE_BUILD_TYPE=$BUILD_TYPE \
		-DIREE_CMAKE_PLUGIN_PATHS=$IREE_CMAKE_PLUGIN_PATH \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON \
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DCMAKE_C_COMPILER=$C_COMPILER \
		-DCMAKE_C_FLAGS=$COMPILER_FLAGS \
		-DCMAKE_CXX_COMPILER=$CXX_COMPILER \
		-DCMAKE_CXX_FLAGS=$COMPILER_FLAGS \
		-DLLVM_ENABLE_TERMINFO=NO \
		-DIREE_EMBEDDED_RELEASE_INFO=ON \
		-DIREE_RELEASE_VERSION=$IREE_COMPILER_VERSION_STRING \
		-DIREE_RELEASE_REVISION=$IREE_GIT_COMMIT
		
	cmake --build $BUILD_DIR/iree-compiler
	
    echo
	echo Building Compiler: Done
	echo
else
	echo
	echo Building Compiler: Incremental build... please do not use for production!
	echo

	cmake --build $BUILD_DIR/iree-compiler
fi


echo =========================================
echo ======== Building NNEMlirTools ==========
echo =========================================

if [ ! -d "$BUILD_DIR/NNEMlirTools" ]; then
	cmake -G Ninja -B $BUILD_DIR/NNEMlirTools $UE_SOURCE_DIR/NNEMlirTools \
		-DCMAKE_BUILD_TYPE=$BUILD_TYPE \
		-DUE_IREE_BUILD_ROOT=$BUILD_DIR \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=OFF \
		-DIREE_HAL_DRIVER_LOCAL_TASK=OFF \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=OFF \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DCMAKE_C_COMPILER=$C_COMPILER \
		-DCMAKE_C_FLAGS=$COMPILER_FLAGS \
		-DCMAKE_CXX_COMPILER=$CXX_COMPILER \
		-DCMAKE_CXX_FLAGS=$COMPILER_FLAGS \
		-DLLVM_ENABLE_TERMINFO=NO 
		
	cmake --build $BUILD_DIR/NNEMlirTools
	
    echo
	echo Building NNEMlirTools: Done
	echo
else
	echo
	echo Building NNEMlirTools: Incremental build... please do not use for production!
	echo

	cmake --build $BUILD_DIR/NNEMlirTools
fi


echo =========================================
echo ======== Building Linux Runtime =========
echo =========================================

if [ ! -d "$BUILD_DIR/iree-runtime-linux" ]; then
	cmake -G Ninja -B $BUILD_DIR/iree-runtime-linux $UE_SOURCE_DIR/Iree/Runtime \
		-DCMAKE_BUILD_TYPE=$BUILD_TYPE \
		-DCMAKE_MODULE_PATH=$UE_CMAKE_DIR \
		-DUE_IREE_BUILD_ROOT=$BUILD_DIR \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON \
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DIREE_BUILD_COMPILER=OFF \
		-DIREE_ENABLE_THREADING=ON \
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF \
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF \
		-DCMAKE_C_COMPILER=$C_COMPILER \
		-DCMAKE_C_FLAGS=$COMPILER_FLAGS \
		-DCMAKE_CXX_COMPILER=$CXX_COMPILER \
		-DCMAKE_CXX_FLAGS=$COMPILER_FLAGS
		
	cmake --build $BUILD_DIR/iree-runtime-linux
	
	echo
	echo Building Linux Runtime: Done
	echo
else
	echo
	echo Building Linux Runtime: Incremental build... please do not use for production!
	echo

	cmake --build $BUILD_DIR/iree-runtime-linux
fi


echo =========================================
echo = Building Linux Runtime - DEV version ==
echo =========================================

TRACING_HEADER=$UE_PLUGIN_ROOT/Source/IREETracing/Internal/IREETracing.h

if [ ! -d "$BUILD_DIR/iree-runtime-linux-dev" ]; then
	cmake -G Ninja -B $BUILD_DIR/iree-runtime-linux-dev $UE_SOURCE_DIR/Iree/Runtime \
		-DCMAKE_BUILD_TYPE=$DEV_BUILD_TYPE \
		-DCMAKE_MODULE_PATH=$UE_CMAKE_DIR \
		-DUE_IREE_BUILD_ROOT=$BUILD_DIR \
		-DUE_IREE_RUNTIME_LIBRARY_NAME=ireert_dev \
		-DIREE_ENABLE_RUNTIME_TRACING=ON \
		-DIREE_TRACING_PROVIDER=Unreal \
		-DIREE_TRACING_PROVIDER_H="$TRACING_HEADER" \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON \
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DIREE_BUILD_COMPILER=OFF \
		-DIREE_ENABLE_THREADING=ON \
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF \
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF \
		-DCMAKE_C_COMPILER=$C_COMPILER \
		-DCMAKE_C_FLAGS=$COMPILER_FLAGS \
		-DCMAKE_CXX_COMPILER=$CXX_COMPILER \
		-DCMAKE_CXX_FLAGS=$COMPILER_FLAGS
		
	cmake --build $BUILD_DIR/iree-runtime-linux-dev
	
	echo
	echo Building Linux Runtime - DEV version: Done
	echo
else
	echo
	echo Building Linux Runtime - DEV version: Incremental build... please do not use for production!
	echo

	cmake --build $BUILD_DIR/iree-runtime-linux-dev
fi