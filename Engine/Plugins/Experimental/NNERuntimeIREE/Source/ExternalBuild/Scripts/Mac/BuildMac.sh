#! /bin/bash

#
# This script builds all the required targets for the NNERuntimeIREE plugin.
#
# If an intermediate step fails, the script exits.
#

set -euo pipefail

if [ -z "$1" ]; then
	echo "Usage: [Script] [Working dir]"
	exit 1
fi

WORKING_DIR=$1

BUILD_DIR=$WORKING_DIR/iree-org
BUILD_SCRIPT_DIR="$(cd -- "$(dirname "${BASH_SOURCE[0]}")" >/dev/null 2>&1 && pwd)"
UE_SOURCE_DIR=$BUILD_SCRIPT_DIR/../../Source
UE_PATCHES_DIR=$BUILD_SCRIPT_DIR/../../Patch
UE_CMAKE_DIR=$BUILD_SCRIPT_DIR/../../CMake
UE_PLUGIN_ROOT=$BUILD_SCRIPT_DIR/../../../..

echo Plugin root dir: "$UE_PLUGIN_ROOT"
echo Build dir: "$BUILD_DIR"
echo Source dir: "$UE_SOURCE_DIR"

IREE_GIT_REPOSITORY=https://github.com/iree-org/iree.git
IREE_GIT_COMMIT=v3.11.0
IREE_THIRD_PARTY_LIBRARIES=("flatcc" "llvm-project" "stablehlo" "torch-mlir" "benchmark" "spirv_cross" "printf")
IREE_COMPILER_VERSION_STRING="IREE-for-UE"

BUILD_TYPE=Release
DEV_BUILD_TYPE=RelWithDebInfo

# Unreal Shader compiler plugin
IREE_CMAKE_PLUGIN_PATH=../../Iree/Compiler/Plugins/Target/UnrealShader

SPIRV_CROSS_DIR=$BUILD_DIR/iree/third_party/spirv_cross

OSX_TARGET=13.0
ARCHITECTURES="arm64;x86_64"

# Common rsync options
# note: -L to copy "resolved" symlinks (e.g. ld.lld)
RSYNC_OPTIONS=(-a -L)
sync() { rsync "${RSYNC_OPTIONS[@]}" -- "$@"; }

echo =========================================
echo ============= Copy Source ===============
echo =========================================

mkdir -p "$WORKING_DIR/iree/compiler"
sync "$UE_SOURCE_DIR/Iree/Compiler/" "$WORKING_DIR/iree/compiler"

echo
echo Copied IREE compiler plugin.
echo


echo =========================================
echo ============= Cloning IREE  =============
echo =========================================

if [ ! -d "$BUILD_DIR/iree" ]; then
	mkdir $BUILD_DIR 2>/dev/null
	cd $BUILD_DIR
	git clone -n $IREE_GIT_REPOSITORY
	cd iree
	echo Using IREE git commit $IREE_GIT_COMMIT
	git checkout $IREE_GIT_COMMIT
	cd third_party
	for D in */; do
		if [ ! -f "$D/CMakeLists.txt" ]; then
			echo -n "" > "$D/CMakeLists.txt"
		fi
	done
	for L in ${IREE_THIRD_PARTY_LIBRARIES[@]}; do
		if [ -d $L ]; then
			rm "$L/CMakeLists.txt"
			git submodule update --init -- $L
			if [ "$L" != "llvm-project" ] && [ -d "$L/third-party" ]; then
				rm -R -f "$L/third-party"
			fi
		fi
	done

	cd ../../..

	echo Check for $SPIRV_CROSS_DIR
	if [ -d "$SPIRV_CROSS_DIR" ]; then
		echo Apply git patch to spirv_cross
		cd $SPIRV_CROSS_DIR
		git apply "$UE_PATCHES_DIR/spirv_cross.patch"
		cd ../../../..
		echo Done.
	fi

	echo
	echo Cloning IREE: Done
	echo
else
	echo
	echo Cloning IREE: Skipped
	echo
fi

echo =========================================
echo ======== Building IREE Compiler =========
echo =========================================

if [ ! -d "$BUILD_DIR/iree-compiler" ]; then
	cmake -G Ninja -B $BUILD_DIR/iree-compiler $BUILD_DIR/iree \
		-DCMAKE_BUILD_TYPE=$BUILD_TYPE \
		-DIREE_CMAKE_PLUGIN_PATHS=$IREE_CMAKE_PLUGIN_PATH \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON \
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DCMAKE_OSX_ARCHITECTURES="$ARCHITECTURES" \
		-DCMAKE_OSX_DEPLOYMENT_TARGET=$OSX_TARGET \
		-DIREE_EMBEDDED_RELEASE_INFO=ON \
		-DIREE_RELEASE_VERSION=$IREE_COMPILER_VERSION_STRING \
		-DIREE_RELEASE_REVISION=$IREE_GIT_COMMIT
		
	cmake --build $BUILD_DIR/iree-compiler
	
	echo
	echo Building Compiler: Done
	echo
else
	echo
	echo Building Compiler: Incremental build... Please do not use for production.
	echo

	cmake --build $BUILD_DIR/iree-compiler
fi


echo =========================================
echo ======== Building NNEMlirTools ==========
echo =========================================

if [ ! -d "$BUILD_DIR/NNEMlirTools" ]; then
	cmake -G Ninja -B $BUILD_DIR/NNEMlirTools $UE_SOURCE_DIR/NNEMlirTools \
		-DCMAKE_BUILD_TYPE=$BUILD_TYPE \
		-DUE_IREE_BUILD_ROOT=$BUILD_DIR \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=OFF \
		-DIREE_HAL_DRIVER_LOCAL_TASK=OFF \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=OFF \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DCMAKE_OSX_ARCHITECTURES="$ARCHITECTURES" \
		-DCMAKE_OSX_DEPLOYMENT_TARGET=$OSX_TARGET
		
	cmake --build $BUILD_DIR/NNEMlirTools
	
	echo
	echo Building NNEMlirTools: Done
	echo
else
	echo
	echo Building NNEMlirTools: Incremental build... Please do not use for production.
	echo

	cmake --build $BUILD_DIR/NNEMlirTools
fi


echo =========================================
echo ========= Building Mac Runtime ==========
echo =========================================

if [ ! -d "$BUILD_DIR/iree-runtime-mac" ]; then
	cmake -G Ninja -B $BUILD_DIR/iree-runtime-mac $UE_SOURCE_DIR/Iree/Runtime \
		-DCMAKE_BUILD_TYPE=$BUILD_TYPE \
		-DCMAKE_MODULE_PATH=$UE_CMAKE_DIR \
		-DUE_IREE_BUILD_ROOT=$BUILD_DIR \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON \
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DIREE_BUILD_COMPILER=OFF \
		-DIREE_ENABLE_THREADING=ON \
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF \
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF \
		-DCMAKE_OSX_ARCHITECTURES="$ARCHITECTURES" \
		-DCMAKE_OSX_DEPLOYMENT_TARGET=$OSX_TARGET
		
	cmake --build $BUILD_DIR/iree-runtime-mac
	
	echo
	echo Building Mac Runtime: Done
	echo
else
	echo
	echo Building Mac Runtime: Incremental build... Please do not use for production.
	echo

	cmake --build $BUILD_DIR/iree-runtime-mac
fi


echo =========================================
echo === Building Mac Runtime - DEV version ==
echo =========================================

TRACING_HEADER=$UE_PLUGIN_ROOT/Source/IREETracing/Internal/IREETracing.h

if [ ! -d "$BUILD_DIR/iree-runtime-mac-dev" ]; then
	cmake -G Ninja -B $BUILD_DIR/iree-runtime-mac-dev $UE_SOURCE_DIR/Iree/Runtime \
		-DCMAKE_BUILD_TYPE=$DEV_BUILD_TYPE \
		-DCMAKE_MODULE_PATH=$UE_CMAKE_DIR \
		-DUE_IREE_BUILD_ROOT=$BUILD_DIR \
		-DUE_IREE_RUNTIME_LIBRARY_NAME=ireert_dev \
		-DIREE_ENABLE_RUNTIME_TRACING=ON \
		-DIREE_TRACING_PROVIDER=Unreal \
		-DIREE_TRACING_PROVIDER_H="$TRACING_HEADER" \
		-DIREE_ENABLE_CPUINFO=OFF \
		-DIREE_BUILD_TESTS=OFF \
		-DIREE_BUILD_SAMPLES=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE=OFF \
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF \
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF \
		-DIREE_HAL_DRIVER_DEFAULTS=OFF \
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON \
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON \
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF \
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON \
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF \
		-DIREE_BUILD_COMPILER=OFF \
		-DIREE_ENABLE_THREADING=ON \
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF \
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF \
		-DCMAKE_OSX_ARCHITECTURES="$ARCHITECTURES" \
		-DCMAKE_OSX_DEPLOYMENT_TARGET=$OSX_TARGET
		
	cmake --build $BUILD_DIR/iree-runtime-mac-dev
	
	echo
	echo Building Mac Runtime - DEV version: Done
	echo
else
	echo
	echo Building Mac Runtime - DEV version: Incremental build... Please do not use for production.
	echo

	cmake --build $BUILD_DIR/iree-runtime-mac-dev
fi