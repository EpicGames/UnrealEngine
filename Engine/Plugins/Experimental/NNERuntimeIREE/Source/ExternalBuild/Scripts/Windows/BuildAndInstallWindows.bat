@ECHO off
SETLOCAL

::
:: Script that calls build and install scripts together with the same arguments.
::
:: If any intermediate step fails, the script (should) exit. Without arguments it will build in
:: NNERuntimeIREE/Source/ExternalBuild/Build and without additional platforms, e.g. Windows only.
::

:: Parse arguments (they will be available to build and install scripts)
CALL "%~dp0_ParseArgs.bat" %* || EXIT /B %ERRORLEVEL%

:: (Debug)print arguments
ECHO:
ECHO Working dir: "%WORKING_DIR%"
FOR %%i IN (%PLATFORM_LIST% "") DO IF NOT "%%~i"=="" (
	ECHO Building for %%~i
)
ECHO:

CALL "%~dp0BuildWindows.bat"
IF ERRORLEVEL 1 GOTO :fail

CALL "%~dp0InstallWindows.bat"
IF ERRORLEVEL 1 GOTO :fail

ENDLOCAL
EXIT /B 0

:fail
ECHO.
ECHO Build and Install script failed. Exiting 1>&2
ENDLOCAL & EXIT /B 1