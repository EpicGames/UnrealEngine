@ECHO off
SETLOCAL

::
:: This script builds all the required targets for the NNERuntimeIREE plugin.
::
:: If an intermediate step fails, the script exits.
::

:: Get arguments
IF NOT DEFINED ARGS_PARSED (
	CALL "%~dp0_ParseArgs.bat" %* || EXIT /B %ERRORLEVEL%
)

IF NOT EXIST "%WORKING_DIR%" (
	ECHO:
	ECHO Warning: Working directory "%WORKING_DIR%" does not exist, creating...
	ECHO:

	MKDIR "%WORKING_DIR%" 2>NUL
)

:: Skip helper function(s)
goto :main

:: Canonicalize paths for Perforce (no '.' or '..' etc.)
:Canonicalize
	FOR %%I IN ("%~1") DO SET "%~2=%%~fI"
	GOTO :eof

:: Generate import library from DLL using llvm tools
:: Usage: CALL :GenerateImportLib "DllPath" "OutputLibPath" "llvm-objdump.exe" "llvm-lib.exe"
:GenerateImportLib
	SETLOCAL
	SET "DLL_PATH=%~1"
	SET "LIB_PATH=%~2"
	SET "OBJDUMP=%~3"
	SET "LLVMLIB=%~4"
	SET "TEMP_DEF=%TEMP%\importlib_%RANDOM%.def"
	SET "TEMP_OUTPUT=%TEMP%\importlib_%RANDOM%.txt"
	IF NOT EXIST "%DLL_PATH%" (
		ECHO Error: DLL not found at "%DLL_PATH%" 1>&2
		ENDLOCAL & EXIT /B 1
	)
	:: Get DLL filename
	FOR %%F IN ("%DLL_PATH%") DO SET "DLL_NAME=%%~nxF"

	:: Create .def file header
	ECHO LIBRARY "%DLL_NAME%"> "%TEMP_DEF%"
	ECHO EXPORTS>> "%TEMP_DEF%"

	:: Dump DLL info to temp file
	"%OBJDUMP%" -p "%DLL_PATH%" > "%TEMP_OUTPUT%" 2>&1

	:: Extract exports from DLL info output and append to .def
	:: llvm-objdump -p output format: "      ordinal    address    name"
	:: We need to extract the function name (third column after whitespace)
	FOR /F "usebackq tokens=1,2,*" %%A IN (`FINDSTR /R "^ *[0-9]" "%TEMP_OUTPUT%"`) DO (
		IF NOT "%%C"=="" ECHO %%C>> "%TEMP_DEF%"
	)

	:: Generate import library
	"%LLVMLIB%" /nologo /def:"%TEMP_DEF%" /out:"%LIB_PATH%" /machine:x64 >NUL 2>&1
	SET "LIB_RC=%ERRORLEVEL%"

	:: Cleanup temp files
	DEL "%TEMP_DEF%" 2>NUL
	DEL "%TEMP_OUTPUT%" 2>NUL
	FOR %%F IN ("%LIB_PATH%") DO DEL "%%~dpnF.exp" 2>NUL
	IF %LIB_RC% NEQ 0 (
		ECHO Error: Failed to generate import library 1>&2
		ENDLOCAL & EXIT /B 1
	)
	ENDLOCAL & EXIT /B 0

:: Main body
:main

SET "BUILD_DIR=%WORKING_DIR%\iree-org"
SET "BUILD_SCRIPT_DIR=%~dp0"
CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\Source" UE_SOURCE_DIR
CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\Patch" UE_PATCHES_DIR
CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\CMake" UE_CMAKE_DIR
CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\..\.." UE_PLUGIN_ROOT

ECHO Plugin root dir: "%UE_PLUGIN_ROOT%"
ECHO Build dir: "%BUILD_DIR%"
ECHO Source dir: "%UE_SOURCE_DIR%"
ECHO Additional runtime platform argument list: %PLATFORM_LIST%

SET IREE_GIT_REPOSITORY=https://github.com/iree-org/iree.git
SET IREE_GIT_COMMIT=v3.11.0
SET IREE_THIRD_PARTY_LIBRARIES=flatcc,llvm-project,stablehlo,torch-mlir,benchmark,spirv_cross,printf
SET IREE_COMPILER_VERSION_STRING="IREE-for-UE"

SET BUILD_TYPE=Release
SET DEV_BUILD_TYPE=RelWithDebInfo

:: Unreal Shader compiler plugin
SET "IREE_CMAKE_PLUGIN_PATH=..\..\Iree\Compiler\Plugins\Target\UnrealShader"

SET "SPIRV_CROSS_DIR=%BUILD_DIR%\iree\third_party\spirv_cross"

SET VC_TOOLSET=14.44.35207
SET WIN_SDK_FOR_IREE=10.0.22621.0
SET WIN_SDK_FOR_RUNTIME=10.0.22621.0

ECHO =========================================
ECHO ============= Copy Source ===============
ECHO =========================================

:: Need to copy at least IREE compiler plugin to avoid path lengths exploding
robocopy "%UE_SOURCE_DIR%\iree\compiler" "%WORKING_DIR%\iree\compiler" /E >NUL
IF ERRORLEVEL 8 GOTO :fail

ECHO:
ECHO Copied IREE compiler plugin.
ECHO:

ECHO =========================================
ECHO ============= Cloning IREE ==============
ECHO =========================================

IF NOT EXIST "%BUILD_DIR%\iree" (
	MKDIR "%BUILD_DIR%" 2>NUL
	CD "%BUILD_DIR%"

	git clone -n %IREE_GIT_REPOSITORY%
	IF ERRORLEVEL 1 GOTO :fail

	CD iree
	
	ECHO Using IREE git commit '%IREE_GIT_COMMIT%'
	git checkout %IREE_GIT_COMMIT%
	IF ERRORLEVEL 1 GOTO :fail
	
	FOR /F "tokens=*" %%F IN ('DIR "%BUILD_DIR%\iree\third_party" /AD /B') DO (
		IF NOT EXIST "%BUILD_DIR%\iree\third_party\%%F\CMakeLists.txt" COPY NUL "%BUILD_DIR%\iree\third_party\%%F\CMakeLists.txt" >NUL
	)
	
	FOR %%L IN (%IREE_THIRD_PARTY_LIBRARIES%) DO (
		IF EXIST "third_party\%%L" (
			DEL ".\third_party\%%L\CMakeLists.txt"
			git submodule update --init -- ""./third_party/%%L"
			IF ERRORLEVEL 1 GOTO :fail
			IF /I NOT "%%L"=="llvm-project" IF EXIST "third_party\%%L\third-party" RMDIR /S /Q "third_party\%%L\third-party"
		)
	)

	CD ../..

	:: Patch IREE
	ECHO Apply git patches to IREE
	CD "%BUILD_DIR%\iree"

	git apply "%UE_PATCHES_DIR%\Windows\iree_device_generic_fma_msvc_fix.patch"
	IF ERRORLEVEL 1 GOTO :fail

	git apply "%UE_PATCHES_DIR%\Windows\flatcc_host_binary_suffix_fix.patch"
	IF ERRORLEVEL 1 GOTO :fail

	CD ..\..
	ECHO Done.


	:: Patch llvm-project
	ECHO Apply git patch to llvm-project
	CD "%BUILD_DIR%\iree\third_party\llvm-project"

	git apply "%UE_PATCHES_DIR%\Windows\llvm-project_compiler-rt_for_msvc_fix.patch"
	IF ERRORLEVEL 1 GOTO :fail

	CD ..\..\..\..
	ECHO Done.

	:: Patch SPIRV-Cross
	ECHO Check for "%SPIRV_CROSS_DIR%"
	IF EXIST "%SPIRV_CROSS_DIR%" (
		ECHO Apply git patch to spirv_cross
		CD "%SPIRV_CROSS_DIR%"

		git apply "%UE_PATCHES_DIR%\spirv_cross.patch"
		IF ERRORLEVEL 1 GOTO :fail

		CD ..\..\..\..
		ECHO Done.
	)

	ECHO:
	ECHO Cloning IREE: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Cloning IREE: Skipped
	ECHO:
)

ECHO =========================================
ECHO ======== Building IREE Compiler =========
ECHO =========================================

IF NOT EXIST "%BUILD_DIR%\iree-compiler" (
	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_IREE% -vcvars_ver=%VC_TOOLSET%
	cmake -G Ninja -B "%BUILD_DIR%\iree-compiler" "%BUILD_DIR%\iree" ^
		-DLLVM_ENABLE_RUNTIMES=compiler-rt ^
		-DCMAKE_BUILD_TYPE=%BUILD_TYPE% ^
		-DIREE_CMAKE_PLUGIN_PATHS="%IREE_CMAKE_PLUGIN_PATH%" ^
		-DIREE_ENABLE_CPUINFO=OFF ^
		-DIREE_BUILD_TESTS=OFF ^
		-DIREE_BUILD_SAMPLES=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF ^
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF ^
		-DIREE_HAL_DRIVER_DEFAULTS=OFF ^
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON ^
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON ^
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF ^
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON ^
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF ^
		-DIREE_EMBEDDED_RELEASE_INFO=ON ^
		-DIREE_RELEASE_VERSION=%IREE_COMPILER_VERSION_STRING% ^
		-DIREE_RELEASE_REVISION=%IREE_GIT_COMMIT%
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	cmake --build "%BUILD_DIR%\iree-compiler"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	:: Build builtins target separately (clang_rt.builtins-x86_64.lib)
	cmake --build "%BUILD_DIR%\iree-compiler" --target builtins
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
	ECHO Building IREE Compiler: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Building IREE Compiler: Incremental build... Please do not use for Release build.
	ECHO:

	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_IREE% -vcvars_ver=%VC_TOOLSET%

	cmake --build "%BUILD_DIR%\iree-compiler"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	:: Build builtins target separately (clang_rt.builtins-x86_64.lib)
	cmake --build "%BUILD_DIR%\iree-compiler" --target builtins
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )
	
	ENDLOCAL
	ECHO:
)

ECHO =========================================
ECHO ===== Generating UCRT Import Library ====
ECHO =========================================

SET "UCRT_DLL_PATH=C:\Program Files (x86)\Windows Kits\10\Redist\%WIN_SDK_FOR_IREE%\ucrt\DLLs\x64\ucrtbase.dll"
SET "UCRT_LIB_PATH=%BUILD_DIR%\implib\ucrtbase_applocal.lib"

IF NOT EXIST "%UCRT_LIB_PATH%" (
	IF NOT EXIST "%BUILD_DIR%\implib" (
		MKDIR "%BUILD_DIR%\implib" 2>NUL
	)
	CALL :GenerateImportLib "%UCRT_DLL_PATH%" "%UCRT_LIB_PATH%" "%BUILD_DIR%\iree-compiler\llvm-project\bin\llvm-objdump.exe" "%BUILD_DIR%\iree-compiler\llvm-project\bin\llvm-lib.exe"
	IF ERRORLEVEL 1 (
		ECHO Could not generate import library. 1>&2
		GOTO :fail
	)
	ECHO:
	ECHO Generating UCRT Import Library: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Generating UCRT Import Library: Skipped
	ECHO:
)

ECHO =========================================
ECHO ======== Building NNEMlirTools ==========
ECHO =========================================

IF NOT EXIST "%BUILD_DIR%\NNEMlirTools" (
	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_IREE% -vcvars_ver=%VC_TOOLSET%
	cmake -G Ninja -B "%BUILD_DIR%/NNEMlirTools" "%UE_SOURCE_DIR%\NNEMlirTools" ^
		-DCMAKE_BUILD_TYPE=%BUILD_TYPE% ^
		-DUE_IREE_BUILD_ROOT="%BUILD_DIR%" ^
		-DIREE_ENABLE_CPUINFO=OFF ^
		-DIREE_BUILD_TESTS=OFF ^
		-DIREE_BUILD_SAMPLES=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF ^
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF ^
		-DIREE_HAL_DRIVER_DEFAULTS=OFF ^
		-DIREE_HAL_DRIVER_LOCAL_SYNC=OFF ^
		-DIREE_HAL_DRIVER_LOCAL_TASK=OFF ^
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF ^
		-DIREE_TARGET_BACKEND_LLVM_CPU=OFF ^
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	cmake --build "%BUILD_DIR%/NNEMlirTools"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
	ECHO Building NNEMlirTools: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Building NNEMlirTools: Incremental build... Please do not use for Release build.
	ECHO:

	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_IREE% -vcvars_ver=%VC_TOOLSET%

	cmake --build "%BUILD_DIR%/NNEMlirTools"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
)

ECHO =========================================
ECHO ======= Building Windows Runtime ========
ECHO =========================================

:: Need to sanitize for cmake
SET "UE_CMAKE_DIR_SANITIZED=%UE_CMAKE_DIR:\=/%"
ECHO Sanitized path: "%UE_CMAKE_DIR_SANITIZED%"

IF NOT EXIST "%BUILD_DIR%\iree-runtime-windows" (
	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_RUNTIME% -vcvars_ver=%VC_TOOLSET%
	cmake -G Ninja -B "%BUILD_DIR%\iree-runtime-windows" "%UE_SOURCE_DIR%\Iree\Runtime" ^
		-DCMAKE_BUILD_TYPE=%BUILD_TYPE% ^
		-DCMAKE_MODULE_PATH="%UE_CMAKE_DIR_SANITIZED%" ^
		-DUE_IREE_BUILD_ROOT="%BUILD_DIR%" ^
		-DIREE_ENABLE_CPUINFO=OFF ^
		-DIREE_BUILD_TESTS=OFF ^
		-DIREE_BUILD_SAMPLES=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF ^
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF ^
		-DIREE_HAL_DRIVER_DEFAULTS=OFF ^
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON ^
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON ^
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF ^
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON ^
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF ^
		-DIREE_BUILD_COMPILER=OFF ^
		-DIREE_ENABLE_THREADING=ON ^
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF ^
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	cmake --build "%BUILD_DIR%\iree-runtime-windows"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
	ECHO Building Windows Runtime: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Building Windows Runtime: Incremental build... Please do not use for Release build.
	ECHO:

	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_RUNTIME% -vcvars_ver=%VC_TOOLSET%

	cmake --build "%BUILD_DIR%\iree-runtime-windows"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
)

ECHO =========================================
ECHO  Building Windows Runtime - DEV version
ECHO =========================================

SET "TRACING_HEADER=%UE_PLUGIN_ROOT%\Source\IREETracing\Internal\IREETracing.h"
SET "IREE_TRACING_PROVIDER_H=%TRACING_HEADER:\=/%"

IF NOT EXIST "%BUILD_DIR%\iree-runtime-windows-dev" (
	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_RUNTIME% -vcvars_ver=%VC_TOOLSET%
	cmake -G Ninja -B "%BUILD_DIR%\iree-runtime-windows-dev" "%UE_SOURCE_DIR%\Iree\Runtime" ^
		-DCMAKE_BUILD_TYPE=%DEV_BUILD_TYPE% ^
		-DCMAKE_MODULE_PATH="%UE_CMAKE_DIR_SANITIZED%" ^
		-DUE_IREE_BUILD_ROOT="%BUILD_DIR%" ^
		-DUE_IREE_RUNTIME_LIBRARY_NAME=ireert_dev ^
		-DIREE_ENABLE_RUNTIME_TRACING=ON ^
		-DIREE_TRACING_PROVIDER=Unreal ^
		-DIREE_TRACING_PROVIDER_H="%IREE_TRACING_PROVIDER_H%" ^
		-DIREE_ENABLE_CPUINFO=OFF ^
		-DIREE_BUILD_TESTS=OFF ^
		-DIREE_BUILD_SAMPLES=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE=OFF ^
		-DIREE_BUILD_BINDINGS_TFLITE_JAVA=OFF ^
		-DIREE_BUILD_ALL_CHECK_TEST_MODULES=OFF ^
		-DIREE_HAL_DRIVER_DEFAULTS=OFF ^
		-DIREE_HAL_DRIVER_LOCAL_SYNC=ON ^
		-DIREE_HAL_DRIVER_LOCAL_TASK=ON ^
		-DIREE_TARGET_BACKEND_DEFAULTS=OFF ^
		-DIREE_TARGET_BACKEND_LLVM_CPU=ON ^
		-DIREE_ERROR_ON_MISSING_SUBMODULES=OFF ^
		-DIREE_BUILD_COMPILER=OFF ^
		-DIREE_ENABLE_THREADING=ON ^
		-DIREE_HAL_EXECUTABLE_LOADER_DEFAULTS=OFF ^
		-DIREE_HAL_EXECUTABLE_PLUGIN_DEFAULTS=OFF
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	cmake --build "%BUILD_DIR%\iree-runtime-windows-dev"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
	ECHO Building Windows Runtime - DEV version: Done
	ECHO:
) ELSE (
	ECHO:
	ECHO Building Windows Runtime - DEV version: Incremental build... Please do not use for Release build.
	ECHO:

	SETLOCAL
	CALL "%VCVARSALL_PATH%" x64 %WIN_SDK_FOR_RUNTIME% -vcvars_ver=%VC_TOOLSET%

	cmake --build "%BUILD_DIR%\iree-runtime-windows-dev"
	IF ERRORLEVEL 1 (ENDLOCAL & GOTO :fail )

	ENDLOCAL

	ECHO:
)

:: Run additional platform build scripts
FOR %%i IN (%PLATFORM_LIST% "") DO IF NOT "%%~i"=="" (
	CALL "%UE_PLUGIN_ROOT%\..\..\..\Platforms\%%~i\Plugins\Experimental\NNERuntimeIREE\Source\ExternalBuild\Scripts\Windows\BuildWindows.bat" "%BUILD_DIR%" "%BUILD_TYPE%" "%UE_CMAKE_DIR_SANITIZED%"
	IF ERRORLEVEL 1 GOTO :fail
)

ENDLOCAL
EXIT /B 0

:fail
ECHO.
ECHO Build script failed. Exiting 1>&2
ENDLOCAL & EXIT /B 1