@ECHO off
SETLOCAL

::
:: This script copies all required files back into the plugin folder tree and adds them to a new Perforce CL.
::
:: If an intermediate step fails, the script exits.
::

:: Get arguments
IF NOT DEFINED ARGS_PARSED (
	CALL "%~dp0_ParseArgs.bat" %* || EXIT /B %ERRORLEVEL%
)

SET "BUILD_DIR=%WORKING_DIR%\iree-org"
SET "BUILD_SCRIPT_DIR=%~dp0"

SET "ROBOCOPY_ARGS=/A-:R /NFL /NDL /NJH /NJS /NP"

:: Skip helper function(s)
goto :main

:: Copy files with robocopy and catch errors
:CP
	SETLOCAL
	robocopy %* %ROBOCOPY_ARGS%
	SET "RC=%ERRORLEVEL%"

	IF %RC% LEQ 3 (
		IF %RC% EQU 0 ( ECHO Up-to-date )
		IF %RC% EQU 1 ( ECHO Copied successfully )
		IF %RC% EQU 2 ( ECHO Warning: extras in destination )
		IF %RC% EQU 3 ( ECHO Copied successfully; extras exist )

		ECHO   "%~1" 1>&2
		ECHO   "%~2" 1>&2
		ENDLOCAL & EXIT /B 0
	)

	ECHO Robocopy failed ^(code %RC%^)
	ECHO   "%~1" 1>&2
	ECHO   "%~2" 1>&2
	ENDLOCAL & EXIT /B 1

:: P4 wrapper
:P4
	p4 %*
	IF ERRORLEVEL 1 GOTO :fail
	GOTO :eof

:: Canonicalize paths for Perforce (no '.' or '..' etc.)
:Canonicalize
	FOR %%I IN ("%~1") DO SET "%~2=%%~fI"
	GOTO :eof

:: Main body
:main

CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\Include" UE_INCLUDE_DIR
CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\Source" UE_SOURCE_DIR
CALL :Canonicalize "%BUILD_SCRIPT_DIR%\..\..\..\.." UE_PLUGIN_ROOT

ECHO Plugin root dir: "%UE_PLUGIN_ROOT%"
ECHO Build dir: "%BUILD_DIR%"
ECHO Source dir: "%UE_SOURCE_DIR%"
ECHO Additional runtime platform argument list: "%PLATFORM_LIST%"

ECHO:

ECHO =========================================
ECHO ======== Copying Compiler Files =========
ECHO =========================================

SET "TARGET_IREE_FOLDER=%UE_PLUGIN_ROOT%\Source\ThirdParty\IREE"
SET "BINARIES_FOLDER=%UE_PLUGIN_ROOT%\Binaries\ThirdParty\IREE\Windows"

CALL :CP "%BUILD_DIR%\iree-compiler\llvm-project\bin" "%BINARIES_FOLDER%" lld-link.exe
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-compiler\llvm-project\lib\clang\23\lib\windows" "%BINARIES_FOLDER%" clang_rt.builtins-x86_64.lib
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-compiler\tools" "%BINARIES_FOLDER%" iree-compile.exe IREECompiler.dll
IF ERRORLEVEL 1 GOTO :fail

ECHO:
ECHO Copying Compiler Files: Done
ECHO:

ECHO =========================================
ECHO ======= Copying Import libraries ========
ECHO =========================================

CALL :CP "%BUILD_DIR%\implib" "%BINARIES_FOLDER%" ucrtbase_applocal.lib

ECHO:
ECHO Copying Import libraries: Done
ECHO:

ECHO =========================================
ECHO ========= Copying Header Files ==========
ECHO =========================================

SET "HEADERS_FOLDER=%TARGET_IREE_FOLDER%\Include"

CALL :CP "%BUILD_DIR%\iree\third_party\flatcc\include\flatcc" "%HEADERS_FOLDER%\flatcc" *.h /MIR
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree\runtime\src\iree" "%HEADERS_FOLDER%\iree" *.h /MIR
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%UE_INCLUDE_DIR%\Clang" "%HEADERS_FOLDER%\Clang" *.h /MIR
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-compiler\runtime\src\iree\base\internal\flatcc" "%HEADERS_FOLDER%\iree\base\internal\flatcc" dummy_reader.h dummy_verifier.h
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-compiler\runtime\src\iree\schemas" "%HEADERS_FOLDER%\iree\schemas" executable_debug_info_reader.h executable_debug_info_verifier.h
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-compiler\compiler\plugins\target\UnrealShader" "%HEADERS_FOLDER%\iree\schemas" unreal_executable_def_reader.h unreal_executable_def_verifier.h
IF ERRORLEVEL 1 GOTO :fail

ECHO:
ECHO Copying Header Files: Done
ECHO:

ECHO =========================================
ECHO ====== Copying NNEMlirTools Files =======
ECHO =========================================

SET "MLIR_TOOLS_INCLUDE_DIR=%UE_PLUGIN_ROOT%\Source\ThirdParty\NNEMlirTools\Internal"
SET "MLIR_TOOLS_BINARIES_DIR=%UE_PLUGIN_ROOT%\Binaries\ThirdParty\NNEMlirTools\Win64"

CALL :CP "%BUILD_DIR%\NNEMlirTools" "%MLIR_TOOLS_BINARIES_DIR%" NNEMlirTools.dll
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%UE_SOURCE_DIR%\NNEMlirTools\NNERuntimeIREE\Include" "%MLIR_TOOLS_INCLUDE_DIR%" *.h /MIR
IF ERRORLEVEL 1 GOTO :fail

ECHO:
ECHO Copying NNEMlirTools Files: Done
ECHO:

ECHO =========================================
ECHO ========= Copying Windows Files =========
ECHO =========================================

SET "LIBRARIES_FOLDER=%TARGET_IREE_FOLDER%\Lib\Windows"

CALL :CP "%BUILD_DIR%\iree-runtime-windows\iree\build_tools\third_party\flatcc" "%LIBRARIES_FOLDER%" flatcc_parsing.lib
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-runtime-windows" "%LIBRARIES_FOLDER%" ireert.lib
IF ERRORLEVEL 1 GOTO :fail

CALL :CP "%BUILD_DIR%\iree-runtime-windows-dev" "%LIBRARIES_FOLDER%" ireert_dev.lib
IF ERRORLEVEL 1 GOTO :fail

ECHO:
ECHO Copying Windows Files: Done
ECHO:

ECHO =========================================
ECHO == Copying Additional Platforms Files ===
ECHO =========================================

SETLOCAL ENABLEDELAYEDEXPANSION
SET "ADDITIONAL_FOLDERS="

FOR %%i IN (%PLATFORM_LIST% "") DO IF NOT "%%~i"=="" (
	CALL :Canonicalize "%UE_PLUGIN_ROOT%\..\..\..\Platforms\%%~i\Plugins\Experimental\NNERuntimeIREE" UE_PLATFORM_PLUGIN_ROOT
	SET "PLATFORM_LIBRARIES_FOLDER=!UE_PLATFORM_PLUGIN_ROOT!\Source\ThirdParty\IREE\Lib\%%~i"
	ECHO Platform Plugin root: "!UE_PLATFORM_PLUGIN_ROOT!"
	ECHO Library dir: "!PLATFORM_LIBRARIES_FOLDER!"

	IF DEFINED ADDITIONAL_FOLDERS (
		SET "ADDITIONAL_FOLDERS=!ADDITIONAL_FOLDERS! "!PLATFORM_LIBRARIES_FOLDER!""
	) ELSE (
		SET "ADDITIONAL_FOLDERS="!PLATFORM_LIBRARIES_FOLDER!""
	)

	CALL :CP "%BUILD_DIR%\iree-runtime-%%~i\iree\build_tools\third_party\flatcc" "!PLATFORM_LIBRARIES_FOLDER!" *.lib *.a
	IF ERRORLEVEL 1 GOTO :fail

	CALL :CP "%BUILD_DIR%\iree-runtime-%%~i" "!PLATFORM_LIBRARIES_FOLDER!" *.lib *.a
	IF ERRORLEVEL 1 GOTO :fail

	SET "PLATFORM_HEADERS_FOLDER_SRC=!UE_PLATFORM_PLUGIN_ROOT!\Source\ExternalBuild\Include"
	IF EXIST !PLATFORM_HEADERS_FOLDER_SRC! (
		SET "PLATFORM_HEADERS_FOLDER_DST=!UE_PLATFORM_PLUGIN_ROOT!\Source\ThirdParty\IREE\Include"

		ECHO:
		ECHO Copy headers:
		ECHO  from "!PLATFORM_HEADERS_FOLDER_SRC!"
		ECHO  to   "!PLATFORM_HEADERS_FOLDER_DST!"

		CALL :CP "!PLATFORM_HEADERS_FOLDER_SRC!" "!PLATFORM_HEADERS_FOLDER_DST!" *.h /MIR
		IF ERRORLEVEL 1 GOTO :fail
	)
)

ECHO All dirs to reconcile: "%ADDITIONAL_FOLDERS%"

ECHO:
ECHO Copying Additional Platforms Files: Done
ECHO:

ECHO =========================================
ECHO ============== Save to CL ===============
ECHO =========================================

:: Check for P4 CLI client
WHERE p4 >NUL 2>&1
IF errorlevel 1 (
	ECHO Error: Perforce CLI ^(p4.exe^) not found in PATH. 1>&2
	GOTO :fail
)

:: Check connection to server
p4 -G info >NUL 2>&1
IF errorlevel 1 (
	ECHO Error: Unable to reach Perforce server. 1>&2
	GOTO :fail
)

:: Check logged in
p4 login -s >NUL 2>&1
IF errorlevel 1 (
	ECHO Starting interactive login...
	p4 login
	if errorlevel 1 (
		ECHO Error: Login failed. 1>&2
		GOTO :fail
	)
)

SET "CL_DESC=IREE install"

:: Create a new CL with the description and NO files in it
for /f "tokens=2" %%C in ('
	p4 --field "Description=%CL_DESC%" --field "Files=" change -o ^
	^| p4 change -i ^
	^| findstr /r "^Change [0-9][0-9]*"
') do (
	SET "CL=%%C"
)

ECHO Created changelist #%CL%

:: Reconcile modified directories
p4 reconcile -a -e -d -c %CL% "%BINARIES_FOLDER%\..."
IF ERRORLEVEL 1 GOTO :fail

p4 reconcile -a -e -d -c %CL% "%HEADERS_FOLDER%\..."
IF ERRORLEVEL 1 GOTO :fail

p4 reconcile -a -e -d -c %CL% "%LIBRARIES_FOLDER%\..."
IF ERRORLEVEL 1 GOTO :fail

p4 reconcile -a -e -d -c %CL% "%MLIR_TOOLS_INCLUDE_DIR%\..."
IF ERRORLEVEL 1 GOTO :fail

p4 reconcile -a -e -d -c %CL% "%MLIR_TOOLS_BINARIES_DIR%\..."
IF ERRORLEVEL 1 GOTO :fail

IF DEFINED ADDITIONAL_FOLDERS (
	FOR %%F IN (!ADDITIONAL_FOLDERS!) DO (
		p4 reconcile -a -e -d -c %CL% "%%~F%\..."
		IF ERRORLEVEL 1 GOTO :fail
	)
)

:: Note: this is required for the ADDITIONAL_FOLDERS list to be up-to-date
ENDLOCAL

ECHO:
ECHO Save to CL: Done
ECHO:

ENDLOCAL
ECHO All work done successfully, bye!
EXIT /B 0

:fail
ECHO.
ECHO Install script failed. Exiting 1>&2
ENDLOCAL & EXIT /B 1