@ECHO OFF
SETLOCAL EnableExtensions EnableDelayedExpansion

:: Usage:
::   _ParseArgs.bat [-dir WorkingDir] [-platform "Platform1 Platform2 ..."]

:: Defaults
FOR %%I IN ("%~dp0..\..\Build") DO SET "WORKING_DIR=%%~fI"
SET "PLATFORM_LIST="

:: UE root (sanitized)
FOR %%I IN ("%~dp0..\..\..\..\..\..\..\..") DO SET "UE5_ROOT=%%~fI"

:: Parse arguments: -dir and -platform
:parse
IF "%~1"=="" GOTO :after_parse

:: -dir PATH
IF /I "%~1"=="-dir" (
	IF "%~2"=="" ( ECHO Error: -dir requires a path.>&2 & EXIT /B 2 )
	SET "WORKING_DIR=%~2"
	SHIFT & SHIFT & GOTO :parse
)

:: -platform LIST (LIST can be "All" or "[Platform1] [Platform2] [Platform3]")
IF /I "%~1"=="-platform" (
	SET "ENGINE_PLATFORMS_DIR=%UE5_ROOT%\Engine\Platforms"
	IF NOT EXIST "%ENGINE_PLATFORMS_DIR%\" EXIT /B 1

	IF "%~2"=="" ( ECHO Error: -platform requires a value.>&2 & EXIT /B 2 )
	IF /I "%~2" == "All" (
		CALL :discover_all
	) else (
		CALL :parse_platforms "%~2" || EXIT /B 1
	)
	SHIFT & SHIFT & GOTO :parse
)

ECHO Error: Unknown option: %~1>&2
EXIT /B 2

:after_parse
:: Canonicalize WORKING_DIR
FOR %%I IN ("%WORKING_DIR%") DO SET "WORKING_DIR=%%~fI"

:: Summary
@REM ECHO WORKING_DIR   = "%WORKING_DIR%"
@REM ECHO UE5_ROOT      = "%UE5_ROOT%"
@REM ECHO PLATFORM_LIST = "%PLATFORM_LIST%"

ENDLOCAL & (
	SET "WORKING_DIR=%WORKING_DIR%"
	SET "UE5_ROOT=%UE5_ROOT%"
	SET "PLATFORM_LIST=%PLATFORM_LIST%"
	SET ARGS_PARSED=1
)
EXIT /B 0


:parse_platforms
:: Expand commas to spaces and add tokens, and handle All
SET "arg=%~1"
SET "arg=%arg:"=%"
SET "arg=%arg:,= %"
FOR %%P IN (%arg%) DO (
	CALL :add_platform "%%~P" || (
		ECHO Error: Platform "%%~P" has no build support or does not exist
		EXIT /B 1
	)
)
EXIT /B 0


:add_platform
:: Add %~1 to PLATFORM_LIST if build script present
SET "PLATFORM=%~1"
IF NOT DEFINED PLATFORM EXIT /B 0

IF NOT EXIST "%ENGINE_PLATFORMS_DIR%\%PLATFORM%\Plugins\Experimental\NNERuntimeIREE\Source\ExternalBuild\Scripts\Windows\BuildWindows.bat" (
	EXIT /B 1
)

IF NOT DEFINED PLATFORM_LIST (
	SET "PLATFORM_LIST=%PLATFORM%"
	EXIT /B 0
)

SET "PLATFORM_LIST=%PLATFORM_LIST% %PLATFORM%"
EXIT /B 0


:discover_all
FOR /D %%D IN ("%ENGINE_PLATFORMS_DIR%\*") DO (
	CALL :add_platform "%%~nxD"
)
EXIT /B 0
