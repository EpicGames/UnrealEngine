iree_add_all_subdirs()

set(SPIRV_CROSS_ENABLE_MSL OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_ENABLE_GLSL ON CACHE BOOL "" FORCE) # Required to enable MSL
set(SPIRV_CROSS_ENABLE_HLSL ON CACHE BOOL "" FORCE) # Required by UnrealShaders

set(SPIRV_CROSS_EXCEPTIONS_TO_ASSERTIONS OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_CLI OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_ENABLE_TESTS OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_SKIP_INSTALL ON CACHE BOOL "" FORCE)

set(SPIRV_CROSS_ENABLE_CPP OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_ENABLE_REFLECT OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_ENABLE_C_API OFF CACHE BOOL "" FORCE)
set(SPIRV_CROSS_ENABLE_UTIL OFF CACHE BOOL "" FORCE)

add_subdirectory(${CMAKE_SOURCE_DIR}/third_party/spirv_cross ${CMAKE_BINARY_DIR}/third_party/spirv_cross EXCLUDE_FROM_ALL)

flatbuffer_c_library(
  NAME
    unreal_executable_def_c_fbs
  SRCS
    "unreal_executable_def.fbs"
  FLATCC_ARGS
    "--reader"
    "--builder"
    "--verifier"
    "--json"
  INCLUDES
    "executable_debug_info.fbs"
  PUBLIC
)

iree_cc_library(
  NAME
    SPIRVToHLSL
  HDRS
    "SPIRVToHLSL.h"
  SRCS
    "SPIRVToHLSL.cpp"
  DEPS
    LLVMSupport
    MLIRSupport
    spirv-cross-hlsl
  PUBLIC
)

iree_cc_library(
  NAME
    UnrealShader
  SRCS
    "UnrealShaderTarget.cpp"
    "UnrealShaderTargetUtils.cpp"
  DEPS
    ::SPIRVToHLSL
    LLVMSupport
    MLIRAsmParser
    MLIRGPUDialect
    MLIRIR
    MLIRSPIRVDialect
    MLIRSPIRVModuleCombiner
    MLIRSPIRVSerialization
    MLIRSupport
    iree::compiler::Codegen::Common
    iree::compiler::Codegen::Dialect::Codegen::IR::IREECodegenDialect
    iree::compiler::Codegen::SPIRV
    iree::compiler::Codegen::Utils
    iree::compiler::Dialect::Encoding::IR
    iree::compiler::Dialect::HAL::Target
    iree::compiler::Dialect::HAL::Utils::ExecutableDebugInfoUtils
    iree::compiler::PluginAPI
    iree::compiler::Utils
    iree::schemas::executable_debug_info_c_fbs
    ::unreal_executable_def_c_fbs
  PUBLIC
)

iree_compiler_register_plugin(
  PLUGIN_ID
    hal_target_unreal_shader
  TARGET
    ::UnrealShader
)