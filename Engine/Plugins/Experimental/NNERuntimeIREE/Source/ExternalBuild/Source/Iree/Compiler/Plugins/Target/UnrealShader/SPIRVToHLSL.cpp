// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPIRVToHLSL.h"

#include "llvm/ADT/STLExtras.h"
#include "llvm/Support/Debug.h"
#include "llvm/Support/raw_ostream.h"

#include <set>
#include <unordered_map>

// Disable exception handling in favor of assertions.
#define SPIRV_CROSS_EXCEPTIONS_TO_ASSERTIONS
#include "third_party/spirv_cross/spirv_hlsl.hpp"

#define DEBUG_TYPE "spirv-to-hlsl"

namespace mlir::iree_compiler {

namespace {

bool GetCommonBaseType(SPIRV_CROSS_NAMESPACE::Compiler& compiler, const SPIRV_CROSS_NAMESPACE::SPIRType &type, SPIRV_CROSS_NAMESPACE::SPIRType::BaseType &baseType)
{
	if (type.basetype == SPIRV_CROSS_NAMESPACE::SPIRType::Struct)
	{
		baseType = SPIRV_CROSS_NAMESPACE::SPIRType::Unknown;
		for (auto &member_type : type.member_types)
		{
			SPIRV_CROSS_NAMESPACE::SPIRType::BaseType member_base;
			if (!GetCommonBaseType(compiler, compiler.get_type(member_type), member_base))
			{
				return false;
			}

			if (baseType == SPIRV_CROSS_NAMESPACE::SPIRType::Unknown)
			{
				baseType = member_base;
			}
			else if (baseType != member_base)
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		baseType = type.basetype;
		return true;
	}
}

}

std::optional<std::pair<HLSLShader, std::string>> crossCompileSPIRVToHLSL(llvm::ArrayRef<uint32_t> spvBinary, StringRef entryPoint, bool useStructuredBuffers) {
	SPIRV_CROSS_NAMESPACE::CompilerHLSL compiler(spvBinary.data(), spvBinary.size());

	// Set Common options (GLSL)
	{
		SPIRV_CROSS_NAMESPACE::CompilerGLSL::Options Options = compiler.get_common_options();
		// Options.emit_push_constant_as_uniform_buffer = true;

		compiler.set_common_options(Options);
	}

	// Set HLSL specific options
	{
		SPIRV_CROSS_NAMESPACE::CompilerHLSL::Options hlslOptions = compiler.get_hlsl_options();
		hlslOptions.shader_model = 62;
		hlslOptions.enable_16bit_types = true;
		hlslOptions.preserve_structured_buffers = useStructuredBuffers;

		compiler.set_hlsl_options(hlslOptions);
	}

	// Do not emit any : register(#) bindings for ALL resource types, and rely on HLSL compiler to assign something.
	compiler.set_resource_binding_flags(SPIRV_CROSS_NAMESPACE::HLSL_BINDING_AUTO_ALL);

	// With HLSL... from spirv_cross/main.cpp
	compiler.remap_num_workgroups_builtin();

	SPIRV_CROSS_NAMESPACE::ShaderResources res = compiler.get_shader_resources();
	if (res.uniform_buffers.size() > 0)
	{
		llvm::errs() << "CrossCompileSPIRVToHLSL: Uniform buffers not supported!\n";
		return std::nullopt;
	}

	std::string shaderParameterMetadata = "Type;Name;ShaderType;Binding;DescriptorSet;ElementType;NumElements\n";

	std::unordered_map<std::string, uint32_t> aliasCounts;

	// Name buffers
	for (auto& ssbo : res.storage_buffers)
	{
		const uint32_t binding = compiler.get_decoration(ssbo.id, spv::DecorationBinding);
		const uint32_t set = compiler.get_decoration(ssbo.id, spv::DecorationDescriptorSet);

		std::string name = "Buffer_" + std::to_string(set) + "_" + std::to_string(binding);

		// Make sure names are unique for aliased buffers
		if (compiler.get_decoration(ssbo.id, spv::DecorationAliased))
		{
			name += "_" + std::to_string(aliasCounts[name]++);
		}

		compiler.set_name(ssbo.id, name);

		if (useStructuredBuffers)
		{
			compiler.set_decoration_string(ssbo.id, spv::DecorationUserTypeGOOGLE, "rwstructuredbuffer");

			shaderParameterMetadata += "BUFFER_UAV;" + name + ";RWStructuredBuffer;" + std::to_string(binding) + ";" + std::to_string(set) + ";;\n";
		}
		else
		{
			shaderParameterMetadata += "BUFFER_UAV;" + name + ";RWByteAddressBuffer;" + std::to_string(binding) + ";" + std::to_string(set) + ";;\n";
		}
	}

	if (res.push_constant_buffers.size() == 1)
	{
		// Push constant buffers have to be flattened as they could not be packed
		compiler.flatten_buffer_block(res.push_constant_buffers[0].id);

		const std::string name = "Constant";

		// For flattened push constant buffers the base type name will be used
		compiler.set_name(res.push_constant_buffers[0].base_type_id, name);

		// Get array size and basic type, for spirv-cross implementation checkout: 
		// void CompilerGLSL::emit_buffer_block_flattened(const SPIRVariable &var)
		// in third_party\iree\third_party\spirv_cross\spirv_glsl.cpp
		//
		// The buffer will be flattened to array of vec4 with common basic type: float32, int32 or uint32.
		const SPIRV_CROSS_NAMESPACE::SPIRType& type = compiler.get_type(res.push_constant_buffers[0].base_type_id);
		const size_t vecSize = 4; // Check flatten code!
		const size_t numVecElements = (compiler.get_declared_struct_size(type) + 15) / 16;

		SPIRV_CROSS_NAMESPACE::SPIRType::BaseType baseType;
		if (!GetCommonBaseType(compiler, type, baseType))
		{
			llvm::errs() << "CrossCompileSPIRVToHLSL: Push constant block contains multiple type, flatten will fail!\n";
			return std::nullopt;
		}
		
		std::string baseTypeStr;
		if (baseType == SPIRV_CROSS_NAMESPACE::SPIRType::Float)
		{
			baseTypeStr = "FLOAT32";
		}
		else if (baseType == SPIRV_CROSS_NAMESPACE::SPIRType::Int)
		{
			baseTypeStr = "INT32";
		}
		else if (baseType == SPIRV_CROSS_NAMESPACE::SPIRType::UInt)
		{
			baseTypeStr = "UINT32";
		}
		else
		{
			llvm::errs() << "CrossCompileSPIRVToHLSL: Basic type " << baseType << " not supported!\n";
			return std::nullopt;
		}

		shaderParameterMetadata += "PARAM_ARRAY;" + name + ";;;;" + baseTypeStr + ";" + std::to_string(numVecElements * vecSize) + "\n";
	}
	else if (res.push_constant_buffers.size() > 1)
	{
		// SPIR-V allows only one push constant block per shader stage, but just to be sure...
		llvm::errs() << "CrossCompileSPIRVToHLSL: Multiple Push Constants not supported!\n";
		return std::nullopt;
	}

	std::string hlslSource = compiler.compile();

	const auto &spirvEntryPoint = compiler.get_entry_point(entryPoint.str(), spv::ExecutionModel::ExecutionModelGLCompute);
	
	return std::make_pair(HLSLShader{std::move(hlslSource), std::move(shaderParameterMetadata)}, spirvEntryPoint.name);
}

} // namespace mlir::iree_compiler
