// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <array>
#include <optional>
#include <string>

#include "llvm/ADT/ArrayRef.h"
#include "llvm/ADT/StringRef.h"
#include "mlir/Support/LLVM.h"

namespace mlir::iree_compiler {

struct HLSLShader {
  std::string source;
  std::string metadata;
};

// Cross compiles SPIR-V into HLSL source code for the
// compute shader with |entryPoint| and returns the HLSL source and the new
// entry point name. Returns std::nullopt on failure.
std::optional<std::pair<HLSLShader, std::string>> crossCompileSPIRVToHLSL(llvm::ArrayRef<uint32_t> spvBinary, StringRef entryPoint, bool useStructuredBuffers = false);

} // namespace mlir::iree_compiler
