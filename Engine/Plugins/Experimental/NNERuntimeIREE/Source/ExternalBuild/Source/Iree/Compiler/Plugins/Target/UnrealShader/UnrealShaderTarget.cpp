// Copyright Epic Games, Inc. All Rights Reserved.

#include "iree/compiler/Codegen/Dialect/Codegen/IR/IREECodegenDialect.h"
#include "iree/compiler/Codegen/SPIRV/Passes.h"
#include "iree/compiler/Codegen/Utils/GPUUtils.h"
#include "iree/compiler/Dialect/Encoding/IR/EncodingDialect.h"
#include "iree/compiler/Dialect/HAL/Target/TargetRegistry.h"
#include "iree/compiler/Dialect/HAL/Utils/ExecutableDebugInfoUtils.h"
#include "iree/compiler/PluginAPI/Client.h"
#include "iree/compiler/Utils/FlatbufferUtils.h"
#include "iree/compiler/Utils/ModuleUtils.h"
#include "llvm/ADT/STLExtras.h"
#include "llvm/Support/CommandLine.h"
#include "llvm/Support/raw_ostream.h"
#include "mlir/AsmParser/AsmParser.h"
#include "mlir/Dialect/GPU/IR/GPUDialect.h"
#include "mlir/Dialect/SPIRV/IR/SPIRVAttributes.h"
#include "mlir/Dialect/SPIRV/IR/SPIRVDialect.h"
#include "mlir/Dialect/SPIRV/IR/SPIRVOps.h"
#include "mlir/Dialect/SPIRV/IR/TargetAndABI.h"
#include "mlir/Dialect/SPIRV/Linking/ModuleCombiner.h"
#include "mlir/IR/Builders.h"
#include "mlir/IR/BuiltinAttributes.h"
#include "mlir/IR/BuiltinOps.h"
#include "mlir/IR/DialectResourceBlobManager.h"
#include "mlir/Target/SPIRV/Serialization.h"
#include "SPIRVToHLSL.h"
#include "unreal_executable_def_builder.h"
#include "UnrealShaderTargetUtils.h"

namespace mlir::iree_compiler::IREE::HAL {

namespace {
struct UnrealShaderTargetOptions {
	// Use vp_android_baseline_2022 profile as the default target--it's a good
	// lowest common denominator to guarantee the generated SPIR-V is widely
	// accepted for now. Eventually we want to use a list for multi-targeting.
	std::string target = "vp_android_baseline_2022";
	bool useStructuredBuffers = false;
	bool indirectBindings = false;

	void bindOptions(OptionsBinder &binder) {
		static llvm::cl::OptionCategory category("UnrealShader HAL Target");
		binder.opt<std::string>(
				"iree-unreal-target", target,
				llvm::cl::desc(
						"Unreal Shader target controlling the SPIR-V environment, which will be "
						"cross-compiled to HLSL and fed to Unreal Engines Shader compiler."
						"Currently the same schemes are supported as for the Vulkan target: 1) LLVM "
						"CodeGen backend style: e.g., 'gfx*' for AMD GPUs and 'sm_*' for "
						"NVIDIA GPUs; 2) architecture code name style: e.g., "
						"'rdna3'/'valhall4'/'ampere'/'adreno' for AMD/ARM/NVIDIA/Qualcomm "
						"GPUs; 3) product name style: 'rx7900xtx'/'rtx4090' for AMD/NVIDIA "
						"GPUs. See "
						"https://iree.dev/guides/deployment-configurations/gpu-vulkan/ for "
						"more details.^"
						"Note: this will likely change in the future!"));
		binder.opt<bool>(
				"iree-unreal-use-structured-buffers", useStructuredBuffers,
				llvm::cl::desc(
						"Force to use the (RW)StructuredBuffer type to be used for all SSBO's."));
		binder.opt<bool>(
				"iree-unreal-experimental-indirect-bindings", indirectBindings,
				llvm::cl::desc(
						"Force indirect bindings for all generated dispatches."));
	}
};
} // namespace

class UnrealTargetDevice : public TargetDevice {
public:
	UnrealTargetDevice(const UnrealShaderTargetOptions & /*options*/) {}

	IREE::HAL::DeviceTargetAttr getDefaultDeviceTarget(MLIRContext *context, const TargetRegistry &targetRegistry) const override
	{
		Builder b(context);

		SmallVector<NamedAttribute> deviceConfigAttrs;
		auto deviceConfigAttr = b.getDictionaryAttr(deviceConfigAttrs);

		SmallVector<NamedAttribute> executableConfigAttrs;
		auto executableConfigAttr = b.getDictionaryAttr(executableConfigAttrs);

		SmallVector<IREE::HAL::ExecutableTargetAttr> executableTargetAttrs;
		targetRegistry.getTargetBackend("unreal-shader")->getDefaultExecutableTargets(context, "unreal", executableConfigAttr, executableTargetAttrs);

		return IREE::HAL::DeviceTargetAttr::get(context, b.getStringAttr("unreal"), deviceConfigAttr, executableTargetAttrs);
	}
};

class UnrealShaderTargetBackend : public TargetBackend {
public:
	UnrealShaderTargetBackend(const UnrealShaderTargetOptions &options) : options_(options) {}

	std::string getLegacyDefaultDeviceID() const override { return "unreal"; }

	void getDefaultExecutableTargets(MLIRContext *context, StringRef deviceID, DictionaryAttr deviceConfigAttr, SmallVectorImpl<IREE::HAL::ExecutableTargetAttr> &executableTargetAttrs) const override {
		executableTargetAttrs.push_back(getExecutableTarget(context, options_.indirectBindings));
	}

	IREE::HAL::ExecutableTargetAttr getExecutableTarget(MLIRContext *context, bool indirectBindings) const {
		Builder b(context);
		SmallVector<NamedAttribute> configItems;

		if (auto target = GPU::UnrealShaderTargetUtils::getTargetDetails(options_.target, context)) {
			addConfigGPUTarget(context, target, configItems);
		} else {
			emitError(b.getUnknownLoc(), "Unknown Unreal target '")
					<< options_.target << "'";
			return nullptr;
		}

		return IREE::HAL::ExecutableTargetAttr::get(
				context, b.getStringAttr("unreal-shader"),
				indirectBindings ? b.getStringAttr("vulkan-spirv-fb-ptr")
												 : b.getStringAttr("vulkan-spirv-fb"),
				b.getDictionaryAttr(configItems));
	}

	void getDependentDialects(DialectRegistry &registry) const override {
		registry.insert<IREE::Codegen::IREECodegenDialect, IREE::Encoding::IREEEncodingDialect, spirv::SPIRVDialect, gpu::GPUDialect, IREE::GPU::IREEGPUDialect>();
	}

	void buildConfigurationPassPipeline(IREE::HAL::ExecutableTargetAttr targetAttr, OpPassManager &passManager) override {
		buildSPIRVCodegenConfigurationPassPipeline(passManager);
	}

	void buildTranslationPassPipeline(IREE::HAL::ExecutableTargetAttr targetAttr, OpPassManager &passManager) override {
		buildSPIRVCodegenPassPipeline(passManager);
	}

	void buildLinkingPassPipeline(OpPassManager &passManager) override {
		buildSPIRVLinkingPassPipeline(passManager);
	}

	LogicalResult serializeExecutable(const SerializationOptions &options, IREE::HAL::ExecutableVariantOp variantOp, OpBuilder &executableBuilder) override {
		// Today we special-case external variants but in the future we could allow
		// for a linking approach allowing both code generation and external .spv
		// files to be combined together.
		if (variantOp.isExternal()) {
			return variantOp.emitError() << "unimplemented serializeExecutable for external .spv files";
		}

		ModuleOp innerModuleOp = variantOp.getInnerModule();
		auto spirvModuleOps = innerModuleOp.getOps<spirv::ModuleOp>();
		if (spirvModuleOps.empty()) {
			return variantOp.emitError() << "should contain some spirv.module ops";
		}

		// Create a list of executable exports (by ordinal) to the SPIR-V module and
		// entry point defining them.
		auto unsortedExportOps = llvm::to_vector(variantOp.getOps<IREE::HAL::ExecutableExportOp>());
		DenseMap<StringRef, std::tuple<IREE::HAL::ExecutableExportOp, uint64_t>> exportOrdinalMap;
		for (auto exportOp : variantOp.getOps<IREE::HAL::ExecutableExportOp>()) {
			uint64_t ordinal = 0;
			if (std::optional<APInt> optionalOrdinal = exportOp.getOrdinal()) {
				ordinal = optionalOrdinal->getZExtValue();
			} else {
				// For executables with only one entry point linking doesn't kick in at
				// all. So the ordinal can be missing for this case.
				if (!llvm::hasSingleElement(unsortedExportOps)) {
					return exportOp.emitError() << "should have ordinal attribute";
				}
			}
			exportOrdinalMap[exportOp.getSymName()] = std::make_tuple(exportOp, ordinal);
		}
		SmallVector<IREE::HAL::ExecutableExportOp> sortedExportOps;
		sortedExportOps.resize(unsortedExportOps.size());
		SmallVector<std::tuple<IREE::HAL::ExecutableExportOp, spirv::ModuleOp, spirv::EntryPointOp>> exportOps;
		exportOps.resize(unsortedExportOps.size());
		for (spirv::ModuleOp spirvModuleOp : spirvModuleOps) {
			// Currently the spirv.module op should only have one entry point.
			auto spirvEntryPoints = spirvModuleOp.getOps<spirv::EntryPointOp>();
			if (!llvm::hasSingleElement(spirvEntryPoints)) {
				// TODO: support multiple entry points. We only need them here to get
				// the module name for dumping files.
				return spirvModuleOp.emitError() << "expected to contain exactly one entry point";
			}
			spirv::EntryPointOp spirvEntryPointOp = *spirvEntryPoints.begin();
			auto [exportOp, ordinal] = exportOrdinalMap.at(spirvEntryPointOp.getFn());
			sortedExportOps[ordinal] = exportOp;
			exportOps[ordinal] = std::make_tuple(exportOp, spirvModuleOp, spirvEntryPointOp);
		}

		FlatbufferBuilder builder;
		iree_hal_unreal_ExecutableDef_start_as_root(builder);

		// Generate optional per-export debug information.
		// May be empty if no debug information was requested.
		auto exportDebugInfos = createExportDefs(options.debugLevel, sortedExportOps, builder);

		// Create a list of all serialized SPIR-V modules.
		// TODO: unique the modules when each contains multiple entry points.
		for (auto [exportOp, spirvModuleOp, spirvEntryPointOp] : exportOps) {
			if (!options.dumpIntermediatesPath.empty()) {
				std::string assembly;
				llvm::raw_string_ostream os(assembly);
				spirvModuleOp.print(os, OpPrintingFlags().useLocalScope());
				dumpDataToPath(options.dumpIntermediatesPath, options.dumpBaseName, spirvEntryPointOp.getFn(), ".spirv.mlir", assembly);
			}

			// Serialize the spirv::ModuleOp into the binary blob.
			SmallVector<uint32_t, 0> spirvBinary;
			if (failed(spirv::serialize(spirvModuleOp, spirvBinary)) || spirvBinary.empty()) {
				return spirvModuleOp.emitError() << "failed to serialize";
			}

			if (!options.dumpBinariesPath.empty()) {
				dumpDataToPath<uint32_t>(options.dumpBinariesPath, options.dumpBaseName, spirvEntryPointOp.getFn(), ".spv", spirvBinary);
			}


			// We can use ArrayRef here given spvBinary reserves 0 bytes on stack.
			ArrayRef spvData(spirvBinary.data(), spirvBinary.size());
			std::optional<std::pair<HLSLShader, std::string>> HLSLResult = crossCompileSPIRVToHLSL(spvData, spirvEntryPointOp.getFn(), options_.useStructuredBuffers);
			if (!HLSLResult) {
				return spirvModuleOp.emitError() << "failed to cross compile SPIR-V to HLSL (" << llvm::join_items("_", options.dumpBaseName, spirvEntryPointOp.getFn()) << ")";
			}

			if (!options.dumpBinariesPath.empty()) {
				dumpDataToPath(options.dumpBinariesPath, options.dumpBaseName, spirvEntryPointOp.getFn(), ".hlsl", HLSLResult->first.source);
				dumpDataToPath(options.dumpBinariesPath, options.dumpBaseName, spirvEntryPointOp.getFn(), ".spmetadata", HLSLResult->first.metadata);
			}
		}

		SmallVector<iree_hal_unreal_UnrealShaderDef_ref_t> unrealShaderRefs;
		for (auto [exportOp, spirvModuleOp, spirvEntryPointOp] : exportOps) {
			auto ordinalAttr = exportOp.getOrdinalAttr();
			assert(ordinalAttr && "ordinals must be assigned");
			int64_t ordinal = ordinalAttr.getInt();
			
			const std::string moduleName = options.dumpBaseName; // TODO 
			const std::string entryPoint(spirvEntryPointOp.getFn());

			// TODO currently followes the debug dump name schema
			const std::string sourceFileName = llvm::join_items("_", moduleName, entryPoint);

			auto sourceFileNameRef = builder.createString(sourceFileName);
			auto moduleNameRef = builder.createString(moduleName);
			auto entryPointRef = builder.createString(entryPoint);

			iree_hal_unreal_UnrealShaderDef_start(builder);
			iree_hal_unreal_UnrealShaderDef_source_file_name_add(builder, sourceFileNameRef);
			iree_hal_unreal_UnrealShaderDef_module_name_add(builder, moduleNameRef);
			iree_hal_unreal_UnrealShaderDef_entry_point_add(builder, entryPointRef);
			iree_hal_unreal_UnrealShaderDef_debug_info_add(builder, exportDebugInfos[ordinal]);
			unrealShaderRefs.push_back(iree_hal_unreal_UnrealShaderDef_end(builder));
		}
		auto unrealShadersRef = builder.createOffsetVecDestructive(unrealShaderRefs);

		// Add top-level executable fields following their order of definition.
		iree_hal_unreal_ExecutableDef_unreal_shaders_add(builder, unrealShadersRef);
		iree_hal_unreal_ExecutableDef_end_as_root(builder);

		// Add the binary data to the target executable.
		auto binaryOp = IREE::HAL::ExecutableBinaryOp::create(
				executableBuilder, variantOp.getLoc(), variantOp.getSymName(),
				variantOp.getTarget().getFormat(),
				builder.getHeaderPrefixedBufferAttr(
						executableBuilder.getContext(),
						/*magic=*/iree_hal_unreal_ExecutableDef_file_identifier,
						/*version=*/0));
		binaryOp.setMimeTypeAttr(executableBuilder.getStringAttr("application/x-flatbuffers"));

		return success();
	}

private:
	const UnrealShaderTargetOptions &options_;
};

namespace {

struct UnrealShaderSession : public PluginSession<UnrealShaderSession, UnrealShaderTargetOptions, PluginActivationPolicy::DefaultActivated> {
	void populateHALTargetDevices(IREE::HAL::TargetDeviceList &targets) {
		// #hal.device.target<"unreal", ...
		targets.add("unreal", [&]() {
			return std::make_shared<UnrealTargetDevice>(options);
		});
	}

	void populateHALTargetBackends(IREE::HAL::TargetBackendList &targets) {
		// #hal.executable.target<"unreal-shader", ...
		targets.add("unreal-shader", [&]() {
			return std::make_shared<UnrealShaderTargetBackend>(options);
		});
	}
};

} // namespace

} // namespace mlir::iree_compiler::IREE::HAL

extern "C" bool iree_register_compiler_plugin_hal_target_unreal_shader(mlir::iree_compiler::PluginRegistrar *registrar) {
	registrar->registerPlugin<mlir::iree_compiler::IREE::HAL::UnrealShaderSession>("hal_target_unreal_shader");
	return true;
}

IREE_DEFINE_COMPILER_OPTION_FLAGS(mlir::iree_compiler::IREE::HAL::UnrealShaderTargetOptions);
