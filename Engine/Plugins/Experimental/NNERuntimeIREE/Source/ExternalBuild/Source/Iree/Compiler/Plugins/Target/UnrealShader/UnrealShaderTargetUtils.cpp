// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealShaderTargetUtils.h"

#include "iree/compiler/Codegen/Dialect/GPU/TargetUtils/KnownTargets.h"

namespace mlir::iree_compiler::IREE::GPU::UnrealShaderTargetUtils
{

static TargetAttr applyUnrealConstraints(TargetAttr targetAttr, MLIRContext *context)
{
	const ComputeBitwidths computeBits = targetAttr.getWgp().getCompute().getValue() & ~ComputeBitwidths::Int8;
	const StorageBitwidths storageBits = targetAttr.getWgp().getStorage().getValue() & ~StorageBitwidths::B8;

	// from Engine\Source\ThirdParty\ShaderConductor\ShaderConductor\External\DirectXShaderCompiler\include\dxc\DXIL\DxilConstants.h
	const int32_t maxWorkgroupMemoryBytes = 8192 * 4; // const unsigned kMaxTGSMSize = 8192 * 4;

	TargetWgpAttr wgp = TargetWgpAttr::get(context,
		ComputeBitwidthsAttr::get(context, computeBits),
		StorageBitwidthsAttr::get(context, storageBits),
		targetAttr.getWgp().getSubgroup(),
		targetAttr.getWgp().getDot(),
		targetAttr.getWgp().getMma(),
		targetAttr.getWgp().getScaledMma(),
		targetAttr.getWgp().getSubgroupSizeChoices(),
		targetAttr.getWgp().getMaxWorkgroupSizes(),
		targetAttr.getWgp().getMaxThreadCountPerWorkgroup(),
		maxWorkgroupMemoryBytes,
		targetAttr.getWgp().getMaxWorkgroupCounts(),
		targetAttr.getWgp().getMaxLoadInstructionBits(),
		targetAttr.getWgp().getSimdsPerWgp(),
		targetAttr.getWgp().getVgprSpaceBits(),
		targetAttr.getWgp().getDmaSizes(),
		targetAttr.getWgp().getWorkgroupMemoryBankCount(),
		targetAttr.getWgp().getExtra()
	);

	return TargetAttr::get(context, targetAttr.getArch(), targetAttr.getFeatures(), wgp, targetAttr.getChip());
}

TargetAttr getTargetDetails(llvm::StringRef target, MLIRContext *context)
{
	// Apple/Metal target — uses IREE's generic Apple GPU profile.
	// The generated SPIR-V is cross-compiled to HLSL which Unreal Engine's
	// shader compiler then targets to Metal.
	if (target == "apple") {
		TargetAttr targetAttr = mlir::iree_compiler::IREE::GPU::getMetalTargetDetails(context);
		return applyUnrealConstraints(targetAttr, context);
	}

	// Vulkan targets (AMD, ARM, NVIDIA, Qualcomm, etc.)
	TargetAttr targetAttr = mlir::iree_compiler::IREE::GPU::getVulkanTargetDetails(target, context);
	if (!targetAttr)
		return nullptr;
	return applyUnrealConstraints(targetAttr, context);
}

} // namespace mlir::iree_compiler::IREE::GPU::UnrealShaderTargetUtils
