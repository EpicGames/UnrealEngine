// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "iree/compiler/Codegen/Dialect/GPU/IR/IREEGPUAttrs.h"
#include "llvm/ADT/StringRef.h"

namespace mlir::iree_compiler::IREE::GPU::UnrealShaderTargetUtils {

// Returns a TargetAttr to describe the details of the given |target|, which can
// be a product name like "rtx3090"/"mali-g710"/"adreno", a microarchitecture
// name like "ampere"/"valhall", or "apple" for Apple/Metal GPUs. Returns a null
// TargetAttr if the given |target| is not recognized.
TargetAttr getTargetDetails(llvm::StringRef target, MLIRContext *context);

} // namespace mlir::iree_compiler::IREE::GPU::UnrealShaderTargetUtils
