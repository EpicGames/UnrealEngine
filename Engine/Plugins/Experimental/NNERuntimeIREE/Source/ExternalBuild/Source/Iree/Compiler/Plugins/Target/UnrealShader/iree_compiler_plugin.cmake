add_subdirectory(${CMAKE_CURRENT_LIST_DIR} target/UnrealShader)