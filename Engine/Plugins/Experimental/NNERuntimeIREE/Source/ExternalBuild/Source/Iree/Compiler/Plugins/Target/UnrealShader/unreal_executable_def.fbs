// Copyright 2019 The IREE Authors
//
// Licensed under the Apache License v2.0 with LLVM Exceptions.
// See https://llvm.org/LICENSE.txt for license information.
// SPDX-License-Identifier: Apache-2.0 WITH LLVM-exception

include "iree/schemas/executable_debug_info.fbs";

namespace iree.hal.unreal;

// 'Unreal Shader v1 Executable'.
file_identifier "USH1";
file_extension "ush1";

// Info about one Unreal shader, compiled from the dumped SPIR-V code, and its
// entry point.
table UnrealShaderDef {
  // String name of the dumped shader source code file.
  source_file_name:string;

  // String name of the shader module.
  module_name:string;

  // String name of the entry point function in the shader module.
  entry_point:string;

  // Optional debug information related to the export.
  debug_info:iree.hal.debug.ExportDef;
}

// A SPIR-V shader module and runtime pipeline layout description.
// This information is used to create the VkShaderModule, VkPipelineLayout, and
// any required VkDescriptorSetLayouts.
table ExecutableDef {
  // Exported functions in canonical executable entry point order.
  unreal_shaders:[UnrealShaderDef];

  // Embedded source files sorted ascending by path.
  source_files:[iree.hal.debug.SourceFileDef];
}

root_type ExecutableDef;
