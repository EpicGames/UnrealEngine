cmake_minimum_required(VERSION 3.24...3.31)

project(NNEMlirTools
	VERSION 0.1
	LANGUAGES CXX)

set(CMAKE_CXX_STANDARD 20)
set(CMAKE_CXX_STANDARD_REQUIRED ON)
set(CMAKE_CXX_EXTENSIONS OFF)

# ────────────────────────────────────────────────────────────────
# IREE
# ────────────────────────────────────────────────────────────────

# IREE paths
set(IREE_SOURCE_DIR ${UE_IREE_BUILD_ROOT}/iree)
set(IREE_BINARY_DIR ${CMAKE_CURRENT_BINARY_DIR}/iree)

# Add IREE
add_subdirectory(${IREE_SOURCE_DIR} ${IREE_BINARY_DIR} EXCLUDE_FROM_ALL)

# ────────────────────────────────────────────────────────────────
# NNEMlirTools shared library
# ────────────────────────────────────────────────────────────────

add_library(NNEMlirTools SHARED)

target_sources(NNEMlirTools
	PUBLIC
		FILE_SET HEADERS
			BASE_DIRS
				NNERuntimeIREE/Include
			FILES
				NNERuntimeIREE/Include/NNEMlirTools.h
				NNERuntimeIREE/Include/NNEMlirTools_cxx_api.h
	PRIVATE
		NNERuntimeIREE/Source/NNEMlirTools.cpp)

target_include_directories(NNEMlirTools PUBLIC
	${CMAKE_CURRENT_SOURCE_DIR}/NNERuntimeIREE/Include)

target_compile_definitions(NNEMlirTools PUBLIC
	NNEMLIR_EXPORT)

target_link_libraries(NNEMlirTools PRIVATE
	ChloOps
	iree::compiler::Dialect::Flow::IR
	iree::compiler::Dialect::HAL::IR::HALDialect
	iree::compiler::Dialect::LinalgExt::IR
	iree::compiler::Dialect::Stream::IR
	iree::compiler::Dialect::Util::IR
	iree::compiler::plugins::input::Torch::torch-mlir-dialects::TMTensorDialectIR
	iree::compiler::plugins::input::Torch::torch-mlir::TorchConversionDialectIR
	iree::compiler::plugins::input::Torch::torch-mlir::TorchDialectIR
	MLIRMLProgramDialect
	MLIRParser
	StablehloOps
	VhloOps)

if(APPLE)
	# Apple ld duplicates libm when MLIR pulls both libm and libSystem.
	target_link_options(NNEMlirTools PRIVATE LINKER:-no_warn_duplicate_libraries)
endif()