// Copyright Epic Games, Inc. All Rights Reserved.

// NNEMlirTools.h - minimal C API for introspecting MLIR modules via LLVM/MLIR.
// The interface is provided as a single, versioned function table to ensure
// binary compatibility across builds and convenient manual loading.
// All objects are opaque handles that are released with the corresponding 'Release' functions.
//
// The API and its handles are not thread-safe.

#pragma once

#include <cstdint>
#include <cstddef>

#if defined(NNEMLIR_EXPORT)
	#if defined(_MSC_VER)
		#define NNEMLIR_API __declspec(dllexport)
		#define NNEMLIR_CALL __cdecl
	#elif defined(__GNUC__)
		#define NNEMLIR_API __attribute__((visibility("default")))
		#define NNEMLIR_CALL
	#else
		#error "Unsupported platform for NNEMLIR export"
	#endif
#else
	#define NNEMLIR_API
	#define NNEMLIR_CALL
#endif

#ifdef __cplusplus
extern "C" {
#endif

/// Success / error codes returned through NNEMlirStatus.
typedef enum {
	NNEMLIR_SUCCESS          = 0,		///< Operation completed successfully.
	NNEMLIR_ERR_PARSE_FAILED = 1,		///< Parsing of file/buffer failed.
	NNEMLIR_ERR_BAD_ARGUMENT = 2,		///< Operation received invalid argument.
	NNEMLIR_ERR_INTERNAL     = 0x7FFF	///< Unexpected internal failure.
} NNEMlirStatusCode;

/// Opaque handles
struct NNEMlirStatus_s;   typedef struct NNEMlirStatus_s*   NNEMlirStatus;		///< Status result of an API call.
struct NNEMlirContext_s;  typedef struct NNEMlirContext_s*  NNEMlirContext;		///< Context, contains MLIRContext with registered dialects.
struct NNEMlirModule_s;   typedef struct NNEMlirModule_s*   NNEMlirModule;		///< Parsed MLIR module.
struct NNEMlirFunction_s; typedef struct NNEMlirFunction_s* NNEMlirFunction;	///< (Public) function within a module.
struct NNEMlirValue_s;    typedef struct NNEMlirValue_s*    NNEMlirValue;		///< Function argument or result.

/// ABI version
#define NNEMLIR_ABI_VERSION 1u

/// Function table
typedef struct NNEMlirApi {
	uint32_t AbiVersion; ///< Must equal NNEMLIR_ABI_VERSION
	uint32_t StructSize; ///< sizeof(NNEMlirApi)
	
	/* Status helpers */
	const char*         (NNEMLIR_CALL *StatusToString)        (NNEMlirStatus);
	NNEMlirStatusCode   (NNEMLIR_CALL *GetStatusCode)         (NNEMlirStatus);
	
	/* Context & parsing */
	NNEMlirContext      (NNEMLIR_CALL *CreateContext)         (void);
	NNEMlirStatus       (NNEMLIR_CALL *ParseModuleFromBuffer) (NNEMlirContext,const char* /* Buffer ptr (null-termination not required) */,size_t /* Buffer size */,NNEMlirModule*);
	NNEMlirStatus       (NNEMLIR_CALL *ParseModuleFromFile)   (NNEMlirContext,const char* /* Path to MLIR file */,NNEMlirModule*);
	
	/* Module-level introspection */
	size_t              (NNEMLIR_CALL *GetPublicFunctionCount)(NNEMlirModule);
	NNEMlirFunction     (NNEMLIR_CALL *GetPublicFunction)     (NNEMlirModule,size_t);
	
	/* Function-level introspection */
	const char*         (NNEMLIR_CALL *GetFunctionName)       (NNEMlirFunction);
	size_t              (NNEMLIR_CALL *GetInputCount)         (NNEMlirFunction);
	size_t              (NNEMLIR_CALL *GetResultCount)        (NNEMlirFunction);
	NNEMlirValue        (NNEMLIR_CALL *GetInputValue)         (NNEMlirFunction,size_t);
	NNEMlirValue        (NNEMLIR_CALL *GetResultValue)        (NNEMlirFunction,size_t);
	
	/* Value-level introspection */
	const char*         (NNEMLIR_CALL *GetValueName)          (NNEMlirValue); ///< nullptr if unnamed.
	const char*         (NNEMLIR_CALL *GetValueTypeText)      (NNEMlirValue); ///< E.g. "tensor<4x16xf32>".
	const char*         (NNEMLIR_CALL *GetElementTypeText)    (NNEMlirValue); ///< Element type only, e.g. "f32".
	size_t              (NNEMLIR_CALL *GetShape)              (NNEMlirValue,int64_t*,size_t);
	
	/* Release opaque handles */
	void (NNEMLIR_CALL *ReleaseStatus)   (NNEMlirStatus);
	void (NNEMLIR_CALL *ReleaseContext)  (NNEMlirContext);
	void (NNEMLIR_CALL *ReleaseModule)   (NNEMlirModule);
	void (NNEMLIR_CALL *ReleaseFunction) (NNEMlirFunction);
	void (NNEMLIR_CALL *ReleaseValue)    (NNEMlirValue);
} NNEMlirApi;

/// Returns a pointer to the global NNEMlirApi or nullptr if the requested
/// minimum ABI version is newer than the implementation.
NNEMLIR_API const NNEMlirApi* NNEMLIR_CALL NNEMlirGetInterface(uint32_t MinVersion);

#ifdef __cplusplus
} /* extern "C" */
#endif