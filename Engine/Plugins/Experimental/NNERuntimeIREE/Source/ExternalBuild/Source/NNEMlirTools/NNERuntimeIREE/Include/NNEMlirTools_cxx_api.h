// Copyright Epic Games, Inc. All Rights Reserved.

// NNEMlirTools C++ API is a header only wrapper around the NNEMlirTools C API.
//
// The C++ API simplifies usage by automatically releasing resources in the destructors.

#pragma once

#include <cassert>
#include <stdexcept>
#include <string>
#include <utility>
#include <vector>

#include "NNEMlirTools.h"

#ifdef NNEMLIR_NO_EXCEPTIONS
// The #ifndef is for the very special case where the user of this library wants to define their own way of handling errors.
// NOTE: This header expects control flow to not continue after calling NNEMLIR_CXX_API_THROW
#ifndef NNEMLIR_CXX_API_THROW
#include <iostream>
#define NNEMLIR_CXX_API_THROW(Message) \
	do { \
		std::cerr << Message << std::endl; \
		abort(); \
	} while (false)
#endif
#else
#define NNEMLIR_CXX_API_THROW(Message) \
	throw std::runtime_error(Message);
#endif

namespace NNEMlirTools {

[[noreturn]] inline void ThrowError(const std::string& Message) {
	NNEMLIR_CXX_API_THROW(Message);
}

// Api manages the NNEMlirApi object, which by design has to be manullay initialized.
// We don't want to link againt the library.
class Api {
public:
	static const Api& Instance() {

		if (!ApiPtr)
		{
			ThrowError("Api::Initialize() has not been called");
		}

		static Api _Api;
		return _Api;
	}

	static void Initialize(const NNEMlirApi* Ptr)
	{
		if (ApiPtr && ApiPtr != Ptr)
		{
			ThrowError("Api already initialised with a different table");
		}

		ApiPtr = Ptr;
	}
	
	const NNEMlirApi* operator->() const { return ApiPtr; }

private:
	static const NNEMlirApi* ApiPtr;

	Api() {

	}

	// Not copyable
	Api(const Api&) = delete;
	Api& operator=(const Api&) = delete;
};

inline const NNEMlirApi* Api::ApiPtr = nullptr;

// Overloaded Release methods, not possible with C API
inline void Release(NNEMlirStatus Sts) { Api::Instance()->ReleaseStatus(Sts); }
inline void Release(NNEMlirContext Ctx) { Api::Instance()->ReleaseContext(Ctx); }
inline void Release(NNEMlirModule Mod) { Api::Instance()->ReleaseModule(Mod); }
inline void Release(NNEMlirFunction Func) { Api::Instance()->ReleaseFunction(Func); }
inline void Release(NNEMlirValue Val) { Api::Instance()->ReleaseValue(Val); }

// Owning pointer holder for the underlying C Objects that is move-only. Similar to std::unique_ptr.
template<class T>
class UniqueHandle
{
public:
	UniqueHandle() noexcept = default;
	explicit UniqueHandle(T Hdl) noexcept : Handle(Hdl) { }

	~UniqueHandle() noexcept
	{
		Reset();
	}

	UniqueHandle(UniqueHandle&& Other) noexcept : Handle(std::exchange(Other.Handle, T{})) { }

	UniqueHandle& operator=(UniqueHandle&& Other) noexcept
	{
		if (this != &Other)
		{
			Reset(Other.Release());
		}
		return *this;
	}

	void Reset(T NewHandle = T{}) noexcept
	{
		if (Handle)
		{
			NNEMlirTools::Release(Handle);
		}

		Handle = NewHandle;
	}

	T Release() noexcept
	{
		return std::exchange(Handle, T{});
	}

	// Move only
	UniqueHandle(const UniqueHandle&) = delete;
	UniqueHandle& operator=(const UniqueHandle&) = delete;

	T Get() const noexcept { return Handle; }
	explicit operator bool() const noexcept { return Handle != T{}; }

private:
	T Handle{};
};

class Function;
class Value;

// Status that is set on success/error and might contain a message.
class Status : public UniqueHandle<NNEMlirStatus>
{
public:
	Status(NNEMlirStatus Sts) noexcept : UniqueHandle<NNEMlirStatus>(Sts)
	{
		assert(Sts && "NNEMlirStatus handle must not be null");
	}

	// True on success or false on error
	explicit operator bool() const
	{
		return Api::Instance()->GetStatusCode(Get()) == NNEMLIR_SUCCESS;
	}

	// Get message (might be only set on error)
	std::string GetMessage() const
	{
		return Api::Instance()->StatusToString(Get());
	}
};

// Context that holds the registered dialects
class Context : public UniqueHandle<NNEMlirContext>
{
public:
	Context() : UniqueHandle<NNEMlirContext>(Api::Instance()->CreateContext())
	{
		if (!Get())
		{
			ThrowError("Failed to create context.");
		}
	}
};

// Module obtained from parsing a MLIR file or buffer
class Module : public UniqueHandle<NNEMlirModule>
{
public:
	static Module ParseFromFile(const Context& Ctx, const char* Path)
	{
		NNEMlirModule Mod{};
		const Status Sts = Api::Instance()->ParseModuleFromFile(Ctx.Get(), Path, &Mod);
		if (!Sts)
		{
			ThrowError(Sts.GetMessage());
		}
		return Module(Mod);
	}

	// Parse MLIR from buffer (not null terminated) and return Module
	static Module ParseFromBuffer(const Context& Ctx, const char* Data, size_t Length)
	{
		NNEMlirModule Mod{};
		const Status Sts = Api::Instance()->ParseModuleFromBuffer(Ctx.Get(), Data, Length, &Mod);
		if (!Sts)
		{
			ThrowError(Sts.GetMessage());
		}
		return Module(Mod);
	}

	// Return public function count
	size_t GetFunctionCount() const
	{
		return Api::Instance()->GetPublicFunctionCount(Get());
	}

	// Retrieve a public function by index
	Function GetFunction(size_t Index) const;

	// Get a vector containing all public functions
	std::vector<Function> GetFunctions() const;

private:
	Module(NNEMlirModule Mod) noexcept : UniqueHandle<NNEMlirModule>(Mod)
	{
		assert(Mod && "NNEMlirModule handle must not be null");
	}
};

// Function in a module
class Function : public UniqueHandle<NNEMlirFunction>
{
public:
	// Get the name of the function
	std::string GetName() const
	{
		const char* Name = Api::Instance()->GetFunctionName(Get());
		return Name ? Name : "";
	}

	// Get the input argument count
	size_t GetInputCount() const
	{
		return Api::Instance()->GetInputCount(Get());
	}

	// Get an input argument by index
	Value GetInput(size_t Index) const;

	// Get a vector with the input arguments
	std::vector<Value> GetInputs() const;

	// Get the result count
	size_t GetResultCount() const
	{
		return Api::Instance()->GetResultCount(Get());
	}

	// Get a result by index
	Value GetResult(size_t Index) const;

	// Get a vector with the results
	std::vector<Value> GetResults() const;

private:
	friend class Module;
	explicit Function(NNEMlirFunction Fn) : UniqueHandle<NNEMlirFunction>(Fn)
	{
		assert(Fn && "NNEMlirFunction handle must not be null");
	}
};

// Argument or result of a function
class Value : public UniqueHandle<NNEMlirValue>
{
public:
	// Get the element type string, e.g. "f32"
	std::string GetElementType() const {
		const char* Type = Api::Instance()->GetElementTypeText(Get());
		return Type ? Type : "";
	}

	// Get the shape type text, e.g. "4x16xf32"
	std::string GetShapeTypeText() const {
		const std::vector<int64_t> Shape  = GetShape();
		const std::string ElementType = GetElementType();

		std::string Result;
		for (size_t i = 0; i < Shape.size(); i++)
		{
			if (i) Result.push_back('x');
			Result += (Shape[i] < 0 ? "?" : std::to_string(Shape[i]));
		}
		if (!Result.empty()) Result.push_back('x');
		Result += ElementType;
		
		return Result;
	}

	// Get the name (empty for results)
	std::string GetName() const {
		const char* Name = Api::Instance()->GetValueName(Get());
		return Name ? Name : "";
	}

	// Get the rank
	size_t GetRank() const
	{
		return Api::Instance()->GetShape(Get(), nullptr, 0);
	}

	// Get the shape
	std::vector<int64_t> GetShape() const {
		const size_t Rank = GetRank();

		std::vector<int64_t> Result(Rank);
		if (Rank)
		{
			Api::Instance()->GetShape(Get(), Result.data(), Rank);
		}
		return Result;
	}

	// Fill in the shape
	size_t GetShape(int64_t* ShapeArray, size_t Capacity) const {
		return Api::Instance()->GetShape(Get(), ShapeArray, Capacity);
	}

private:
	friend Function;
	explicit Value(NNEMlirValue Val) : UniqueHandle<NNEMlirValue>(Val)
	{
		assert(Val && "NNEMlirValue handle must not be null");
	}
};

inline Function Module::GetFunction(size_t Index) const
{
	if (Index >= GetFunctionCount())
	{
		ThrowError("GetFunction index out of range");
	}
	return Function(Api::Instance()->GetPublicFunction(Get(), Index));
}

inline std::vector<Function> Module::GetFunctions() const
{
	const size_t NumFuns = GetFunctionCount();

	std::vector<Function> Result;
	Result.reserve(NumFuns);
	for (size_t i = 0; i < NumFuns; i++)
	{
		Result.push_back(GetFunction(i));
	}

	return Result;
}

inline Value Function::GetInput(size_t Index) const
{
	if (Index >= GetInputCount())
	{
		ThrowError("input index out of range");
	}

	return Value(Api::Instance()->GetInputValue(Get(), Index));
}

inline std::vector<Value> Function::GetInputs() const
{
	const size_t NumInputs = GetInputCount();

	std::vector<Value> Result;
	Result.reserve(NumInputs);
	for (size_t i = 0; i < NumInputs; i++)
	{
		Result.push_back(GetInput(i));
	}

	return Result;
}

inline Value Function::GetResult(size_t Index) const
{
	if (Index >= GetResultCount())
	{
		ThrowError("result index out of range");
	}

	return Value(Api::Instance()->GetResultValue(Get(), Index));
}

inline std::vector<Value> Function::GetResults() const
{
	const size_t NumResults = GetResultCount();

	std::vector<Value> Result;
	Result.reserve(NumResults);
	for (size_t i = 0; i < NumResults; i++)
	{
		Result.push_back(GetResult(i));
	}

	return Result;
}

} // namespace NNEMlirTools
