// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEMlirTools.h"

#include "iree/compiler/Dialect/Flow/IR/FlowDialect.h"
#include "iree/compiler/Dialect/HAL/IR/HALDialect.h"
#include "iree/compiler/Dialect/LinalgExt/IR/LinalgExtDialect.h"
#include "iree/compiler/Dialect/Stream/IR/StreamDialect.h"
#include "iree/compiler/Dialect/Util/IR/UtilDialect.h"
#include "llvm/ADT/StringRef.h"
#include "llvm/Support/CommandLine.h"
#include "llvm/Support/ErrorOr.h"
#include "llvm/Support/MemoryBuffer.h"
#include "llvm/Support/raw_ostream.h"
#include "llvm/Support/SourceMgr.h"
#include "mlir/Dialect/Arith/IR/Arith.h"
#include "mlir/Dialect/Func/IR/FuncOps.h"
#include "mlir/Dialect/Linalg/IR/Linalg.h"
#include "mlir/Dialect/Math/IR/Math.h"
#include "mlir/Dialect/MLProgram/IR/MLProgram.h"
#include "mlir/Dialect/SCF/IR/SCF.h"
#include "mlir/Dialect/Shape/IR/Shape.h"
#include "mlir/Dialect/Tensor/IR/Tensor.h"
#include "mlir/Dialect/Tosa/IR/TosaOps.h"
#include "mlir/Interfaces/FunctionInterfaces.h"
#include "mlir/IR/BuiltinOps.h"
#include "mlir/IR/MLIRContext.h"
#include "mlir/IR/SymbolTable.h"
#include "mlir/Parser/Parser.h"
#include "stablehlo/dialect/ChloOps.h"
#include "stablehlo/dialect/StablehloOps.h"
#include "stablehlo/dialect/VhloOps.h"
#include "torch-mlir-dialects/Dialect/TMTensor/IR/TMTensorDialect.h"
#include "torch-mlir/Dialect/Torch/IR/TorchDialect.h"
#include "torch-mlir/Dialect/Torch/IR/TorchTypes.h"
#include "torch-mlir/Dialect/TorchConversion/IR/TorchConversionDialect.h"

//
// Object hierarchy & lifetime:
// * NNEMlirStatus: wrapper around status code & message.
// * NNEMlirContext: owns shared pointer to mlir::MLIRContext. Multiple modules may share it.
// * NNEMlirModule: owns shared pointer to mlir::ModuleOp and copy of context shared pointer to keep mlir::MLIRContext alive.
// * NNEMlirFunction: non-owning view of function operation (implements mlir::FunctionOpInterface). But also has context and module shared pointers.
// * NNEMlirValue: non-owning view of BlockArgument (input) or Type (result). Also has context and module shared pointers.
//
// All handles are created with 'new' and released with matching 'NNEMlirReleaseXYZ()'. All handles are safe to destroy in any order
// as every handle keeps a shared reference to the context and module it was created from.
//
// Returned c-string pointers have the same lifetime as the object that returned it.

struct NNEMlirStatus_s
{
	NNEMlirStatusCode Code;
	std::string Message;

	explicit NNEMlirStatus_s(NNEMlirStatusCode C) : Code(C) { }

	NNEMlirStatus_s(NNEMlirStatusCode C, const std::string& Msg) : Code(C), Message(Msg) { }
};

// Create NNEMlirStatus helper macros
#define MAKE_STATUS_SUCCESS() \
	new NNEMlirStatus_s(NNEMLIR_SUCCESS);

#define MAKE_STATUS_FROM_CODE(Code) \
	new NNEMlirStatus_s(Code);

#define MAKE_ERROR_STATUS(Code, Message) \
	new NNEMlirStatus_s(Code, Message);

struct NNEMlirContext_s
{
	std::shared_ptr<mlir::MLIRContext> Context;
	
	explicit NNEMlirContext_s(std::shared_ptr<mlir::MLIRContext> Ctx) : Context(std::move(Ctx)) { }
};

struct NNEMlirModule_s
{
	// Keep mlir context and module alive
	std::shared_ptr<mlir::MLIRContext> Context;
	std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Module;

	llvm::SmallVector<mlir::FunctionOpInterface, 8> PublicFuncs;

	NNEMlirModule_s(std::shared_ptr<mlir::MLIRContext> Ctx, std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Mod)
		: Context(std::move(Ctx)), Module(std::move(Mod))
	{
		mlir::ModuleOp ModOp = Module->get();
		PublicFuncs.clear();

		for (mlir::FunctionOpInterface Func : ModOp.getOps<mlir::FunctionOpInterface>())
		{
			if (auto Sym = llvm::dyn_cast<mlir::SymbolOpInterface>(Func.getOperation()); Sym && Sym.isPublic())
			{
				PublicFuncs.push_back(Func);
			}
		}
	}
};

struct NNEMlirFunction_s
{
	// Keep mlir context and module alive
	std::shared_ptr<mlir::MLIRContext> Context;
	std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Module;

	mlir::FunctionOpInterface Func;

	std::string CachedName;

	NNEMlirFunction_s(std::shared_ptr<mlir::MLIRContext> Ctx, std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Mod, mlir::FunctionOpInterface Fn)
		: Context(Ctx), Module(std::move(Mod)), Func(Fn) { }
};

struct NNEMlirValue_s
{
	// Keep mlir context and module alive
	std::shared_ptr<mlir::MLIRContext> Context;
	std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Module;

	mlir::Value Argument;
	mlir::Type ResultType;
	bool bIsResult; // Either 'Argument' or 'ResultType' is used

	std::string CachedName;
	std::string CachedType;
	std::string CachedElementType;

	NNEMlirValue_s(std::shared_ptr<mlir::MLIRContext> Ctx, std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Mod, mlir::Value Arg)
		: Context(Ctx), Module(std::move(Mod)), Argument(Arg), bIsResult(false) { }

	NNEMlirValue_s(std::shared_ptr<mlir::MLIRContext> Ctx, std::shared_ptr<mlir::OwningOpRef<mlir::ModuleOp>> Mod, mlir::Type ResTy)
		: Context(Ctx), Module(std::move(Mod)), ResultType(ResTy), bIsResult(true) { }
};

const char* NNEMlirStatusToString(NNEMlirStatus Status)
{
	if (!Status->Message.empty())
	{
		return Status->Message.c_str();
	}

	switch(Status->Code)
	{
		case NNEMLIR_SUCCESS:			return "success";
		case NNEMLIR_ERR_PARSE_FAILED:	return "failed to parse";
		case NNEMLIR_ERR_BAD_ARGUMENT:	return "bad argument";
		case NNEMLIR_ERR_INTERNAL:		return "internal error";
		default:						return "unknown status";
	}
}

NNEMlirStatusCode NNEMlirGetStatusCode(NNEMlirStatus Status)
{
	return Status->Code;
}

NNEMlirContext NNEMlirCreateContext()
{
	std::shared_ptr<mlir::MLIRContext> Context = std::make_shared<mlir::MLIRContext>();

	// Register Built-in dialects, selected according IREE docs:
	// https://iree.dev/reference/extensions/#1-target-iree-input-dialects
	Context->getOrLoadDialect<mlir::arith::ArithDialect>();
	Context->getOrLoadDialect<mlir::func::FuncDialect>();
	Context->getOrLoadDialect<mlir::linalg::LinalgDialect>();
	Context->getOrLoadDialect<mlir::math::MathDialect>();
	Context->getOrLoadDialect<mlir::scf::SCFDialect>();
	Context->getOrLoadDialect<mlir::tensor::TensorDialect>();
	// ...add more dialects?

	// Register IREE's three main input dialects:
	//
	// 1. StableHLO
	Context->getOrLoadDialect<mlir::chlo::ChloDialect>();
	Context->getOrLoadDialect<mlir::shape::ShapeDialect>();
	Context->getOrLoadDialect<mlir::stablehlo::StablehloDialect>();
	Context->getOrLoadDialect<mlir::vhlo::VhloDialect>();
	// 2. Torch-MLIR
	Context->getOrLoadDialect<mlir::iree_compiler::IREE::LinalgExt::IREELinalgExtDialect>();
	Context->getOrLoadDialect<mlir::ml_program::MLProgramDialect>();
	Context->getOrLoadDialect<mlir::torch::TMTensor::TMTensorDialect>();
	Context->getOrLoadDialect<mlir::torch::Torch::TorchDialect>();
	Context->getOrLoadDialect<mlir::torch::TorchConversion::TorchConversionDialect>();
	// 3. TOSA
	Context->getOrLoadDialect<mlir::tosa::TosaDialect>();

	// Register IREE internal dialect(s)
	// Specifically because torch might convert into them as input to the IREE compilation pipeline:
	// iree-org\iree\compiler\plugins\input\Torch\PluginRegistration.cpp
	Context->getOrLoadDialect<mlir::iree_compiler::IREE::Flow::FlowDialect>();
	Context->getOrLoadDialect<mlir::iree_compiler::IREE::HAL::HALDialect>();
	Context->getOrLoadDialect<mlir::iree_compiler::IREE::Stream::StreamDialect>();
	Context->getOrLoadDialect<mlir::iree_compiler::IREE::Util::UtilDialect>();

	return new NNEMlirContext_s(std::move(Context));
}

void NNEMlirReleaseStatus(NNEMlirStatus Status)
{
	if (!Status) return;
	delete Status;
}

void NNEMlirReleaseContext(NNEMlirContext Context)
{
	if (!Context) return;
	delete Context;
}

void NNEMlirReleaseModule(NNEMlirModule Module)
{
	if (!Module) return;
	delete Module;
}

void NNEMlirReleaseFunction(NNEMlirFunction Func)
{
	if (!Func) return;
	delete Func;
}

void NNEMlirReleaseValue(NNEMlirValue Value)
{
	if (!Value) return;
	delete Value;
}

NNEMlirStatus ParseSourceFile(NNEMlirContext Context, std::unique_ptr<llvm::MemoryBuffer> MemoryBuffer, NNEMlirModule* OutModule)
{
	llvm::SourceMgr SourceMgr;
	SourceMgr.AddNewSourceBuffer(std::move(MemoryBuffer), llvm::SMLoc());

	std::string DiagnosticsStr;
	llvm::raw_string_ostream DiagOS(DiagnosticsStr);
	mlir::SourceMgrDiagnosticHandler SourceMgrHandler(SourceMgr, Context->Context.get(), DiagOS);

	const mlir::ParserConfig Config(Context->Context.get(), false);
	mlir::OwningOpRef<mlir::ModuleOp> ModuleRef = mlir::parseSourceFile<mlir::ModuleOp>(SourceMgr, Config);
	if (!ModuleRef)
	{
		DiagOS.flush();

		return MAKE_ERROR_STATUS(NNEMLIR_ERR_PARSE_FAILED, DiagnosticsStr.empty() ? "Failed to parse from buffer" : DiagnosticsStr);
	}

	*OutModule = new NNEMlirModule_s(Context->Context, std::make_shared<mlir::OwningOpRef<mlir::ModuleOp>>(std::move(ModuleRef)));

	return MAKE_STATUS_SUCCESS();
}

NNEMlirStatus NNEMlirParseModuleFromBuffer(NNEMlirContext Context, const char* Data, size_t Length, NNEMlirModule* OutModule)
{
	if (!Context || !Data || Length == 0 || !OutModule)
	{
		return MAKE_STATUS_FROM_CODE(NNEMLIR_ERR_BAD_ARGUMENT);
	}

	llvm::StringRef Source(Data, Length);

	// IREE 3.4.0, note: we need to copy and let add a traling NUL. Can be removed if LLVM/MLIR fixed that.
	llvm::ErrorOr<std::unique_ptr<llvm::MemoryBuffer>> FileOrErr = llvm::MemoryBuffer::getMemBufferCopy(Source, "<Buffer>");
	if (std::error_code ErrorCode = FileOrErr.getError())
	{
		return MAKE_ERROR_STATUS(NNEMLIR_ERR_INTERNAL, "Could not get memory buffer copy: " + ErrorCode.message());
	}

	return ParseSourceFile(Context, std::move(*FileOrErr), OutModule);
}

NNEMlirStatus NNEMlirParseModuleFromFile(NNEMlirContext Context, const char* Path, NNEMlirModule* OutModule)
{
	if (!Context || !Path || !OutModule)
	{
		return MAKE_STATUS_FROM_CODE(NNEMLIR_ERR_BAD_ARGUMENT);
	}

	llvm::ErrorOr<std::unique_ptr<llvm::MemoryBuffer>> FileOrErr = llvm::MemoryBuffer::getFileOrSTDIN(Path);
	if (std::error_code ErrorCode = FileOrErr.getError())
	{
		return MAKE_ERROR_STATUS(NNEMLIR_ERR_INTERNAL, "Could not open input file: " + ErrorCode.message());
	}

	return ParseSourceFile(Context, std::move(*FileOrErr), OutModule);
}

size_t NNEMlirGetPublicFunctionCount(NNEMlirModule Module)
{
	if (!Module) return 0;

	return Module->PublicFuncs.size();
}

NNEMlirFunction NNEMlirGetPublicFunction(NNEMlirModule Module, size_t Index)
{
	if (!Module) return nullptr;
	if (Index >= Module->PublicFuncs.size()) return nullptr;

	const mlir::FunctionOpInterface Func = Module->PublicFuncs[Index];

	return new NNEMlirFunction_s(Module->Context, Module->Module, Func);
}

const char* NNEMlirGetFunctionName(NNEMlirFunction Func)
{
	if (!Func) return nullptr;

	if (Func->CachedName.empty())
	{
		Func->CachedName = Func->Func.getName().str();
	}

	return Func->CachedName.c_str();
}

size_t NNEMlirGetInputCount(NNEMlirFunction Func)
{
	if (!Func) return 0;

	return Func->Func.getNumArguments();
}

size_t NNEMlirGetResultCount(NNEMlirFunction Func)
{
	if (!Func) return 0;

	return Func->Func.getNumResults();
}

NNEMlirValue NNEMlirGetInputValue(NNEMlirFunction Func, size_t Index)
{
	if (!Func) return nullptr;
	if (Index >= Func->Func.getNumArguments()) return nullptr;

	const mlir::Value Argument = Func->Func.getArgument(Index);

	return new NNEMlirValue_s(Func->Context, Func->Module, Argument);
}

NNEMlirValue NNEMlirGetResultValue(NNEMlirFunction Func, size_t Index)
{
	if (!Func) return nullptr;
	if (Index >= Func->Func.getNumResults()) return nullptr;

	const mlir::Type ResultType = Func->Func.getResultTypes()[Index];

	return new NNEMlirValue_s(Func->Context, Func->Module, ResultType);
}

const char* NNEMlirGetValueName(NNEMlirValue Value)
{
	if (!Value) return nullptr;
	if (Value->bIsResult) return nullptr;

	if (Value->CachedName.empty())
	{
		const mlir::BlockArgument BlockArg = mlir::cast<mlir::BlockArgument>(Value->Argument);
		mlir::FunctionOpInterface Func = mlir::cast<mlir::FunctionOpInterface>(BlockArg.getOwner()->getParentOp());

		const unsigned int Index = BlockArg.getArgNumber();
		if (auto Attr = Func.getArgAttrOfType<mlir::StringAttr>(Index, "llvm.name"))
		{
			Value->CachedName = Attr.getValue().str();
		}
	}

	return Value->CachedName.empty() ? nullptr : Value->CachedName.c_str();
}

const char* NNEMlirGetValueTypeText(NNEMlirValue Value)
{
	if (!Value) return nullptr;

	if (Value->CachedType.empty())
	{
		mlir::Type Type = Value->bIsResult ? Value->ResultType : Value->Argument.getType();

		std::string TypeStr;
		llvm::raw_string_ostream Os(TypeStr);
		Type.print(Os);

		Value->CachedType = TypeStr;
	}

	return Value->CachedType.c_str();
}

const char* NNEMlirGetElementTypeText(NNEMlirValue Value)
{
	if (!Value) return nullptr;

	if (Value->CachedElementType.empty())
	{
		mlir::Type Type = Value->bIsResult ? Value->ResultType : Value->Argument.getType();

		// 1. Built-in tensors / memrefs etc.
		if (auto Shaped = mlir::dyn_cast<mlir::ShapedType>(Type))
		{
			Type = Shaped.getElementType();
		} // 2. Torch-MLIR tensors
		else if (auto TorchTensor = mlir::dyn_cast<mlir::torch::Torch::BaseTensorType>(Type))
		{
			if (auto DType = TorchTensor.getOptionalDtype())
			{
				Type = DType;
			}
		}
		// else{} 3. Something else

		std::string TypeStr;
		llvm::raw_string_ostream Os(TypeStr);
		Type.print(Os);

		Value->CachedElementType = TypeStr;
	}

	return Value->CachedElementType.c_str();
}

size_t NNEMlirGetShape(NNEMlirValue Value, int64_t* Dims, size_t Capacity)
{
	if (!Value)
	{
		return 0;
	}

	mlir::Type Type = Value->bIsResult ? Value->ResultType : Value->Argument.getType();

	// 1. Built-in tensors / memrefs etc.
	if (auto Shaped = mlir::dyn_cast<mlir::ShapedType>(Type))
	{
		if (!Shaped.hasRank())
		{
			return 0;
		}

		const size_t Rank = Shaped.getRank();
		if (!Dims || Capacity == 0)
		{
			return Rank;
		}

		const size_t Num = std::min(Rank, Capacity);
		const llvm::ArrayRef<int64_t> Shape = Shaped.getShape();
		for (size_t i = 0; i < Num; ++i)
		{
			int64_t Dim = Shape[i];
			if (mlir::ShapedType::isDynamic(Dim))
			{
				Dim = -1;
			}
			Dims[i] = Dim;
		}

		return Num;
	}

	// 2. Torch-MLIR tensors
	if (auto TorchTensor = mlir::dyn_cast<mlir::torch::Torch::BaseTensorType>(Type))
	{
		if (auto OptSizes = TorchTensor.getOptionalSizes())
		{
			const llvm::ArrayRef<int64_t> Shape = *OptSizes;
			const size_t Rank = Shape.size();

			if (!Dims || Capacity == 0)
			{
				return Rank;
			}

			const size_t Num = std::min(Rank, Capacity);
			for (size_t i = 0; i < Num; ++i)
			{
				int64_t Dim = Shape[i];
				if (Dim == mlir::torch::Torch::kUnknownSize)
				{
					Dim = -1;
				}
				Dims[i] = Dim;
			}

			return Num;
		}

		// unranked !torch.tensor
		return 0;
	}

	// 3. Something else
	return 0;
}

static const NNEMlirApi G_Api = {
	NNEMLIR_ABI_VERSION,
	sizeof(NNEMlirApi),
	&NNEMlirStatusToString,
	&NNEMlirGetStatusCode,
	&NNEMlirCreateContext,
	&NNEMlirParseModuleFromBuffer,
	&NNEMlirParseModuleFromFile,
	&NNEMlirGetPublicFunctionCount,
	&NNEMlirGetPublicFunction,
	&NNEMlirGetFunctionName,
	&NNEMlirGetInputCount,
	&NNEMlirGetResultCount,
	&NNEMlirGetInputValue,
	&NNEMlirGetResultValue,
	&NNEMlirGetValueName,
	&NNEMlirGetValueTypeText,
	&NNEMlirGetElementTypeText,
	&NNEMlirGetShape,
	&NNEMlirReleaseStatus,
	&NNEMlirReleaseContext,
	&NNEMlirReleaseModule,
	&NNEMlirReleaseFunction,
	&NNEMlirReleaseValue
};

extern "C"
{

const NNEMlirApi* NNEMlirGetInterface(uint32_t MinVersion)
{
	return (MinVersion <= G_Api.AbiVersion) ? &G_Api : nullptr;
}

}