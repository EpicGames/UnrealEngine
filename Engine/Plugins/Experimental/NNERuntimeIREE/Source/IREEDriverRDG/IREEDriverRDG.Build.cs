// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IREEDriverRDG : ModuleRules
{
	public IREEDriverRDG(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"IREE",
				"NNE",
				"Projects",
				"RenderCore",
				"RHI",
				"IREEUtils",
				"NNERuntimeIREEShader"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Json",
					"ShaderCompilerCommon"
				}
			);
		}

		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64)
		{
			PublicDefinitions.Add("WITH_IREE_DRIVER_RDG");
			PrivateDefinitions.Add("IREE_DRIVER_RDG_VERBOSITY=0");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicDefinitions.Add("WITH_IREE_DRIVER_RDG");
			PrivateDefinitions.Add("IREE_DRIVER_RDG_VERBOSITY=0");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicDefinitions.Add("WITH_IREE_DRIVER_RDG");
			PrivateDefinitions.Add("IREE_DRIVER_RDG_VERBOSITY=0");
		}
	}
}
