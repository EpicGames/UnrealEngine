// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"
#include "RHIShaderPlatform.h"
#include "Serialization/JsonSerializerMacros.h"

#include "IREECompilerRDG.generated.h"

USTRUCT()
struct FIREECompilerRDGExecutableData
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	TArray<uint8> Data;
};

USTRUCT()
struct FIREECompilerRDGBuildTargetResult
{
	GENERATED_BODY()

	UPROPERTY()
	FString ShaderPlatform;

	UPROPERTY()
	TArray<FIREECompilerRDGExecutableData> Executables;

	int64 DataSize;
	TArray64<uint8> VmfbData;

	bool Serialize(FArchive& Ar)
	{
		// Serialize normal UPROPERTY tagged data
		UScriptStruct* Struct = FIREECompilerRDGBuildTargetResult::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), Struct, nullptr);

		if (Ar.IsLoading())
		{
			Ar << DataSize;
			VmfbData.SetNumUninitialized(DataSize);
			Ar.Serialize((void*)VmfbData.GetData(), DataSize);
		}
		else if (Ar.IsSaving())
		{
			DataSize = VmfbData.Num();
			Ar << DataSize;
			Ar.Serialize((void*)VmfbData.GetData(), DataSize);
		}

		return true;
	}
};

template<>
struct TStructOpsTypeTraits<FIREECompilerRDGBuildTargetResult> : public TStructOpsTypeTraitsBase2<FIREECompilerRDGBuildTargetResult>
{
	enum
	{
		WithSerializer = true
	};
};

USTRUCT()
struct FIREECompilerRDGResult
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FIREECompilerRDGBuildTargetResult> BuildTargetResults;
};

#ifdef WITH_IREE_DRIVER_RDG
#if WITH_EDITOR

namespace UE::IREE::Compiler::RDG
{
	struct FBuildTarget : public FJsonSerializable
	{
		FString ShaderPlatform;
		FString CompilerArguments;
		FString LinkerArguments;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("ShaderPlatform", ShaderPlatform);
			JSON_SERIALIZE("CompilerArguments", CompilerArguments);
		END_JSON_SERIALIZER
	};

	struct FBuildConfig : public FJsonSerializable
	{
		TArray<FString> ImporterCommand;
		FString ImporterArguments;
		TArray<FString> CompilerCommand;
		TArray<FBuildTarget> BuildTargets;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE_ARRAY("ImporterCommand", ImporterCommand);
			JSON_SERIALIZE("ImporterArguments", ImporterArguments);
			JSON_SERIALIZE_ARRAY("CompilerCommand", CompilerCommand);
			JSON_SERIALIZE_ARRAY_SERIALIZABLE("Targets", BuildTargets, FBuildTarget);
		END_JSON_SERIALIZER
	};

	class IREEDRIVERRDG_API FCompiler
	{
	private:
		FCompiler(const ITargetPlatform* TargetPlatform, const FString& InImporterCommand, const FString& InImporterArguments, const FString& InCompilerCommand, const FString& InSharedLibExt, TConstArrayView<FBuildTarget> InBuildTargets, const FMD5Hash& InHash);

	public:
		~FCompiler() {};
		static TUniquePtr<FCompiler> Make(const ITargetPlatform* TargetPlatform);

	public:
		const FMD5Hash& GetHash() const { return Hash; }
		bool ImportOnnx(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TArray64<uint8>& OutMlirData);
		bool CompileMlir(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TConstArrayView<EShaderPlatform> ShaderPlatforms, FIREECompilerRDGResult& OutCompilerResult);

	private:
		const ITargetPlatform* TargetPlatform;
		FString ImporterCommand;
		FString ImporterArguments;
		FString CompilerCommand;
		TArray<FBuildTarget> BuildTargets;
		const FMD5Hash Hash;
	};
} // namespace UE::IREE::Compiler::RDG

#endif // WITH_EDITOR
#endif // WITH_IREE_DRIVER_RDG