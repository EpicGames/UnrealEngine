// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#endif // PLATFORM_MICROSOFT
THIRD_PARTY_INCLUDES_START
#include "iree/hal/api.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // PLATFORM_MICROSOFT

#include "RenderGraphFwd.h"

namespace UE::IREE::HAL::RDG
{

IREEDRIVERRDG_API iree_status_t DeviceCreate(iree_string_view_t Identifier, iree_allocator_t HostAllocator, const TMap<FString, TConstArrayView<uint8>>& Executables, iree_hal_device_t** OutDevice);

IREEDRIVERRDG_API iree_status_t BufferWrapRDG(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_memory_type_t MemoryType, iree_hal_memory_access_t AllowedAccess, iree_hal_buffer_usage_t AllowedUsage, iree_device_size_t AllocationSize, iree_device_size_t ByteOffset, iree_device_size_t ByteLength, FRDGBuilder* GraphBuilder, FRDGBufferRef RDGBuffer, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer);
IREEDRIVERRDG_API FRDGBufferRef BufferRDGBuffer(iree_hal_buffer_t* Buffer, FRDGBuilder* GraphBuilder);

IREEDRIVERRDG_API void DeviceAllocatorSetGraphBuilder(iree_hal_allocator_t* DeviceAllocator, FRDGBuilder& GraphBuilder);
IREEDRIVERRDG_API FRDGBuilder& DeviceAllocatorGetGraphBuilder(iree_hal_allocator_t* DeviceAllocator);
IREEDRIVERRDG_API void DeviceAllocatorResetGraphBuilder(iree_hal_allocator_t* DeviceAllocator);

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG