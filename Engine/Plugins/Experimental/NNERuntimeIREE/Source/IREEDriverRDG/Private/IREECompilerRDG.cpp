// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREECompilerRDG.h"

#ifdef WITH_IREE_DRIVER_RDG
#if WITH_EDITOR

// UE shader compilation
#include "NNERuntimeIREEShaderShared.h"
#include "ShaderParameterMetadataBuilder.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "IREEDriverRDGLog.h"
#include "IREEDriverRDGShaderParametersMetadata.h"
#include "IREEUtils.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNE.h"
#include "Serialization/ArchiveSavePackageDataBuffer.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/SharedPointer.h"

namespace UE::IREE::Compiler
{
	namespace RDG
	{
		namespace Private
		{
			bool CompileAndSerializeShaderFromHLSLSource(const FString& HlslFilepath, EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform, const FString& Outdir, bool bDumpHLSL, FIREECompilerRDGExecutableData& ExecutableData)
			{
				UE_LOGF(LogIREEDriverRDG, Log, "Process %ls", *HlslFilepath);

				FString HlslSource;
				if (!FFileHelper::LoadFileToString(HlslSource, *HlslFilepath))
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "Could not load file to string: %ls", *HlslFilepath);
					return false;
				}

				if (FDataDrivenShaderPlatformInfo::GetSupportsRealTypes(ShaderPlatform) == ERHIFeatureSupport::RuntimeGuaranteed)
				{
					HlslSource.ReplaceInline(TEXT("min16float2"), TEXT("float16_t2"));
				}

				if (FDataDrivenShaderPlatformInfo::GetIsLanguageSony(ShaderPlatform))
				{
					// No support for C99 long long and long double data types.
					HlslSource.ReplaceInline(TEXT("ull"), TEXT("ul"));
				}

				const FString MetadataFilepath = FPaths::GetBaseFilename(HlslFilepath, false) + ".spmetadata";

				FIREEDriverRDGShaderParametersMetadata IREEShaderParametersMetadata;
				if (!HAL::RDG::BuildIREEShaderParametersMetadata(MetadataFilepath, IREEShaderParametersMetadata))
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "Could not build shader parameter metadata!");
					return false;
				}

				TUniquePtr<FNNERuntimeIREEShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations = MakeUnique<FNNERuntimeIREEShaderParametersMetadataAllocations>();
				FShaderParametersMetadata* ShaderParametersMetadata = HAL::RDG::BuildShaderParametersMetadata(IREEShaderParametersMetadata, *ShaderParameterMetadataAllocations);

				const FStaticFeatureLevel FeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
				const FString HlslEntryPointName = TEXT("main");

				TUniquePtr<FNNERuntimeIREEResource> KernelResource = MakeUnique<FNNERuntimeIREEResource>();
				KernelResource->SetupResource(
					FeatureLevel.operator ERHIFeatureLevel::Type(),
					HlslEntryPointName,
					HlslEntryPointName,
					FString(),
					HlslSource,
					MoveTemp(ShaderParameterMetadataAllocations),
					ShaderParametersMetadata,
					FName(),
					{}
				);

				if (!KernelResource->CacheShaders(ShaderPlatform, TargetPlatform, true, true))
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "Failed to compile FNNERuntimeIREEResource [%ls] for platform [%ls].", *KernelResource->GetFriendlyName(), *LegacyShaderPlatformToShaderFormat(ShaderPlatform).ToString());
					return false;
				}

#ifdef IREE_DEBUG_OUTPUT_SHADERMAP
				const FString ShaderMapOutputFilepath = FPaths::Combine(Outdir, FPaths::GetBaseFilename(HlslFilepath, true) + ".ireeshadermap");
#endif

				TOptional<FArchiveSavePackageDataBuffer> ArchiveSavePackageData;

				TArray/*64*/<uint8> ResultData;
				FMemoryWriter/*64*/ Writer(ResultData, /*bIsPersitent =*/ true);

				if (TargetPlatform != nullptr)
				{
					ArchiveSavePackageData.Emplace(TargetPlatform);
					Writer.SetSavePackageData(&ArchiveSavePackageData.GetValue());
				}

				FIREEDriverRDGShaderParametersMetadata::StaticStruct()->SerializeBin(Writer, &IREEShaderParametersMetadata);

				// note: Ignore return value for now since errors should already be reported
				/*bool bShaderMapValid = */ KernelResource->SerializeShaderMap(Writer);

#ifdef IREE_DEBUG_OUTPUT_SHADERMAP
				FFileHelper::SaveArrayToFile(ResultData, *ShaderMapOutputFilepath);
#endif

				ExecutableData.Name = FPaths::GetBaseFilename(HlslFilepath, true);
				ExecutableData.Data = MoveTemp(ResultData);

				return true;
			}

			TArray64<uint8> ReadVmfb(const FString& InVmfbPath)
			{
				check(!InVmfbPath.IsEmpty());
			
				TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*InVmfbPath, 0));
				if (!Reader)
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG failed to open the vmfb data file '%ls'", *InVmfbPath);
					return {};
				}
				int64 DataSize = Reader->TotalSize();
				if (DataSize < 1)
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG vmfb data file '%ls' is empty", *InVmfbPath);
					return {};
				}

				TArray64<uint8> Result;
				Result.SetNumUninitialized(DataSize);

				Reader->Serialize(Result.GetData(), DataSize);

				return Result;
			}
		} // Private

		FCompiler::FCompiler(const ITargetPlatform* InTargetPlatform, const FString& InImporterCommand, const FString& InImporterArguments, const FString& InCompilerCommand, const FString& InSharedLibExt, TConstArrayView<FBuildTarget> InBuildTargets, const FMD5Hash& InHash) : TargetPlatform(InTargetPlatform), ImporterCommand(InImporterCommand), ImporterArguments(InImporterArguments), CompilerCommand(InCompilerCommand), BuildTargets(InBuildTargets), Hash(InHash)
		{
			
		}

		TUniquePtr<FCompiler> FCompiler::Make(const ITargetPlatform* TargetPlatform)
		{
			SCOPED_NAMED_EVENT_TEXT("FCompiler::Make", FColor::Magenta);

			using namespace Private;

			const FString TargetPlatformName = TargetPlatform ? TargetPlatform->IniPlatformName() : UGameplayStatics::GetPlatformName();

			FString PluginDir = FPaths::ConvertRelativePathToFull(*IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetBaseDir());
			FString BuildConfigFileName = FString("IREERDG_") + UGameplayStatics::GetPlatformName() + "_To_" + TargetPlatformName + ".json";
			TArray<FString> BuildConfigFilePaths =
			{
				FPaths::Combine(FPaths::ConvertRelativePathToFull(*FPaths::ProjectConfigDir()), BuildConfigFileName),
				FPaths::Combine(PluginDir, "Config", BuildConfigFileName),
				FPaths::Combine(FPaths::ConvertRelativePathToFull(*FPaths::EngineDir()), "Platforms", TargetPlatformName, "Plugins", UE_PLUGIN_NAME, "Config", BuildConfigFileName),
				FPaths::Combine(FPaths::ConvertRelativePathToFull(*FPaths::EngineDir()), "Platforms", TargetPlatformName, "Plugins", "Experimental", UE_PLUGIN_NAME, "Config", BuildConfigFileName)
			};

			FString ImporterCommand;
			FString ImporterArguments;
			FString CompilerCommand;
			FString SharedLibExt;
			TArray<FBuildTarget> BuildTargets;
			FMD5Hash Hash;
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			for (FString BuildConfigFilePath : BuildConfigFilePaths)
			{
				if (PlatformFile.FileExists(*BuildConfigFilePath))
				{
					FString BuildConfigFileString;
					if (FFileHelper::LoadFileToString(BuildConfigFileString, *BuildConfigFilePath))
					{
						TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(BuildConfigFileString);
						TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
						FBuildConfig BuildConfig;
						if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid() && BuildConfig.FromJson(JsonObject))
						{
							if (BuildConfig.BuildTargets.IsEmpty())
							{
								UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not find targets in %ls", *BuildConfigFilePath);
								continue;
							}

							FString TmpImporterCommand;
							for (int32 i = 0; i < BuildConfig.ImporterCommand.Num(); i++)
							{
								if (IREEUtils::ResolveEnvironmentVariables(BuildConfig.ImporterCommand[i]))
								{
									BuildConfig.ImporterCommand[i].ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
									BuildConfig.ImporterCommand[i].ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());
									if (PlatformFile.FileExists(*BuildConfig.ImporterCommand[i]))
									{
										TmpImporterCommand = BuildConfig.ImporterCommand[i];
										break;
									}
								}
								else
								{
									UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not replace environment variables in %ls", *BuildConfig.ImporterCommand[i]);
								}
							}
							if (TmpImporterCommand.IsEmpty())
							{
								UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not find the importer executable in %ls", *BuildConfigFilePath);
								continue;
							}

							FString TmpCompilerCommand;
							for (int32 i = 0; i < BuildConfig.CompilerCommand.Num(); i++)
							{
								if (IREEUtils::ResolveEnvironmentVariables(BuildConfig.CompilerCommand[i]))
								{
									BuildConfig.CompilerCommand[i].ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
									BuildConfig.CompilerCommand[i].ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());
									if (PlatformFile.FileExists(*BuildConfig.CompilerCommand[i]))
									{
										TmpCompilerCommand = BuildConfig.CompilerCommand[i];
										break;
									}
								}
								else
								{
									UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not replace environment variables in %ls", *BuildConfig.CompilerCommand[i]);
								}
							}
							if (TmpCompilerCommand.IsEmpty())
							{
								UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not find the compiler executable in %ls", *BuildConfigFilePath);
								continue;
							}

							ImporterCommand = TmpImporterCommand;
							ImporterArguments = BuildConfig.ImporterArguments;
							CompilerCommand = TmpCompilerCommand;
							BuildTargets = BuildConfig.BuildTargets;

							FMD5 HashBuilder{};
							bool bHashSuccess = true;
							UE::IREEUtils::HashAppendString(HashBuilder, BuildConfigFileString);
							bHashSuccess &= UE::IREEUtils::HashAppendFileStat(HashBuilder, ImporterCommand);
							bHashSuccess &= UE::IREEUtils::HashAppendFileStat(HashBuilder, CompilerCommand);
							Hash.Set(HashBuilder);
							if (!bHashSuccess)
							{
								UE_LOGF(LogIREEDriverRDG, Warning, "UNNERuntimeIREERDG failed to create a valid compiler hash, while loading %ls", *BuildConfigFilePath);
							}
							break;
						}
						else
						{
							UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not parse build config file %ls", *BuildConfigFilePath);
						}
					}
					else
					{
						UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not read build config file %ls", *BuildConfigFilePath);
					}
				}
			}
			if (CompilerCommand.IsEmpty() || BuildTargets.IsEmpty())
			{
				return TUniquePtr<FCompiler>();
			}
			return TUniquePtr<FCompiler>(new FCompiler(TargetPlatform, ImporterCommand, ImporterArguments, CompilerCommand, SharedLibExt, BuildTargets, Hash));
		}

		bool FCompiler::ImportOnnx(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TArray64<uint8>& OutMlirData)
		{
			return IREEUtils::ImportOnnx(ImporterCommand, ImporterArguments, InFileData, InModelName, InOutputDir, OutMlirData);
		}

		bool FCompiler::CompileMlir(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TConstArrayView<EShaderPlatform> ShaderPlatforms, FIREECompilerRDGResult& OutCompilerResult)
		{
			SCOPED_NAMED_EVENT_TEXT("FCompiler::CompileMlir", FColor::Magenta);

			using namespace Private;

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			FString InputFilePath = FPaths::Combine(InOutputDir, InModelName) + ".mlir";
			if (!PlatformFile.FileExists(*InputFilePath))
			{
				SCOPED_NAMED_EVENT_TEXT("InputFile", FColor::Magenta);

				FFileHelper::SaveArrayToFile(InFileData, *InputFilePath);
			}

			const FString PluginDir = FPaths::ConvertRelativePathToFull(*IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetBaseDir());

			bool bResult = true;
			for (EShaderPlatform ShaderPlatform : ShaderPlatforms)
			{
				const FString ShaderPlatformName = LexToString(ShaderPlatform);

				const FBuildTarget *Target = BuildTargets.FindByPredicate([ShaderPlatformName] (const FBuildTarget &Element) { return Element.ShaderPlatform == ShaderPlatformName; });
				if (!Target)
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not find build target for shader platform %ls", *ShaderPlatformName);
					// Don't fail, model data will not be available...
					// bResult = false;
					continue;
				}

				FString IntermediateDirPath = FPaths::Combine(InOutputDir, Target->ShaderPlatform);
				PlatformFile.CreateDirectoryTree(*IntermediateDirPath);
				FString IntermediateFilePathNoExt = FPaths::Combine(IntermediateDirPath, InModelName);
				FString VmfbFilePath = IntermediateFilePathNoExt + ".vmfb";

				FString CompilerArguments = Target->CompilerArguments;
				if (!IREEUtils::ResolveEnvironmentVariables(CompilerArguments))
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG could not replace environment variables in %ls", *Target->CompilerArguments);
					bResult = false;
					continue;
				}
				CompilerArguments.ReplaceInline(*FString("${BINARIES_PATH}"), *(FString("\"") + IntermediateDirPath + "\""));
				CompilerArguments.ReplaceInline(*FString("${VMFB_PATH}"), *(FString("\"") + VmfbFilePath + "\""));
				CompilerArguments.ReplaceInline(*FString("${INPUT_PATH}"), *(FString("\"") + InputFilePath + "\""));

				IREEUtils::RunCommand(CompilerCommand, CompilerArguments, PluginDir, IntermediateFilePathNoExt + "_compile-log.txt");

				if (!PlatformFile.FileExists(*VmfbFilePath))
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "IREECompilerRDG failed to compile the model \"%ls\" using the command:", *InputFilePath);
					UE_LOGF(LogIREEDriverRDG, Warning, "\"%ls\" %ls", *CompilerCommand, *CompilerArguments);
					bResult = false;
					continue;
				}

				TArray64<uint8> VmfbData = ReadVmfb(VmfbFilePath);
				if (VmfbData.IsEmpty())
				{
					// Log in function...
					bResult = false;
					continue;
				}

				TArray<FString> HlslFilenames;
				PlatformFile.FindFiles(HlslFilenames, *IntermediateDirPath, TEXT(".hlsl"));
				if (HlslFilenames.IsEmpty())
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "No HLSL shader files generated!");
				}

				const FStaticFeatureLevel FeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
				ERHIFeatureLevel::Type FeatureLevelType = FeatureLevel.operator ERHIFeatureLevel::Type();

				if (FeatureLevelType != ERHIFeatureLevel::Type::SM6)
				{
					const FString FeatureLevelName = LexToString(FeatureLevelType);
					UE_LOGF(LogIREEDriverRDG, Warning, "Shader platform %ls: Minimum RHI Feature level is SM6 (desired feature level %ls)!", *ShaderPlatformName, *FeatureLevelName);
					
					continue;
				}

				if (FDataDrivenShaderPlatformInfo::GetSupportsRealTypes(ShaderPlatform) == ERHIFeatureSupport::Unsupported)
				{
					UE_LOGF(LogIREEDriverRDG, Warning, "Shader platform %ls does not support 16-bit types!", *ShaderPlatformName);

					continue;
				}

				TArray<FIREECompilerRDGExecutableData> Executables;
				for (const FString& HlslFilename : HlslFilenames)
				{
					FIREECompilerRDGExecutableData& ExecutableData = Executables.Emplace_GetRef();
					if (!CompileAndSerializeShaderFromHLSLSource(HlslFilename, ShaderPlatform, TargetPlatform, IntermediateDirPath, true, ExecutableData))
					{
						bResult = false;
					}
				}

				FIREECompilerRDGBuildTargetResult Result;
				Result.ShaderPlatform = Target->ShaderPlatform;
				Result.Executables = MoveTemp(Executables);
				Result.DataSize = VmfbData.Num();
				Result.VmfbData = MoveTemp(VmfbData);
			
				OutCompilerResult.BuildTargetResults.Add(MoveTemp(Result));
			}

			bResult &= !OutCompilerResult.BuildTargetResults.IsEmpty();
			
			if (!bResult)
			{
				OutCompilerResult.BuildTargetResults.Empty();
			}
			
			return bResult;
		}
	} // namespace RDG
} // namespace UE::IREE::Compiler

#endif // WITH_EDITOR
#endif // WITH_IREE_DRIVER_RDG