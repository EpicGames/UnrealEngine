// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGBuffer.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "RenderGraphUtils.h"
#include "IREEDriverRDGLog.h"

namespace UE::IREE::HAL::RDG
{

namespace Private
{

class FBuffer
{
public:
	static iree_status_t BufferWrap(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_memory_type_t MemoryType, iree_hal_memory_access_t AllowedAccess, iree_hal_buffer_usage_t AllowedUsage, iree_device_size_t AllocationSize, iree_device_size_t ByteOffset, iree_device_size_t ByteLength, const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls PooledBuffer Size %d", StringCast<TCHAR>(__FUNCTION__).Get(), PooledBuffer->Desc.GetSize());
#endif
		check(DeviceAllocator);
		check(PooledBuffer.IsValid());
		check(OutBuffer);

		FBuffer* Buffer = nullptr;
		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, sizeof(*Buffer), (void**)&Buffer));
		iree_hal_buffer_initialize(iree_hal_buffer_placement_undefined(), &Buffer->Base, AllocationSize, ByteOffset, ByteLength, MemoryType, AllowedAccess, AllowedUsage, &FBuffer::VTable, &Buffer->Base);
		Buffer->HostAllocator = HostAllocator;
		Buffer->PooledBuffer = PooledBuffer;
		Buffer->UserReleaseCallback = UserReleaseCallback;

		check(Buffer->Invariant());

		*OutBuffer = (iree_hal_buffer_t*)Buffer;

#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "--> Created iree_hal_buffer_t 0x%x", (uint64)*OutBuffer);
#endif
		
		return iree_ok_status();
	}

	static iree_status_t BufferWrap(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_memory_type_t MemoryType, iree_hal_memory_access_t AllowedAccess, iree_hal_buffer_usage_t AllowedUsage, iree_device_size_t AllocationSize, iree_device_size_t ByteOffset, iree_device_size_t ByteLength, FRDGBuilder* GraphBuilder, FRDGBufferRef RDGBuffer, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls RDGBuffer Size %d", StringCast<TCHAR>(__FUNCTION__).Get(), RDGBuffer->Desc.GetSize());
#endif
		check(DeviceAllocator);
		check(GraphBuilder);
		check(RDGBuffer != FRDGBufferRef{});
		check(OutBuffer);

		FBuffer* Buffer = nullptr;
		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, sizeof(*Buffer), (void**)&Buffer));
		iree_hal_buffer_initialize(iree_hal_buffer_placement_undefined(), &Buffer->Base, AllocationSize, ByteOffset, ByteLength, MemoryType, AllowedAccess, AllowedUsage, &FBuffer::VTable, &Buffer->Base);
		Buffer->HostAllocator = HostAllocator;
		Buffer->GraphBuilder = GraphBuilder;
		Buffer->RDGBuffer = RDGBuffer;
		Buffer->UserReleaseCallback = UserReleaseCallback;

		check(Buffer->Invariant());

		*OutBuffer = (iree_hal_buffer_t*)Buffer;

#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "--> Created iree_hal_buffer_t 0x%x", (uint64)*OutBuffer);
#endif
		
		return iree_ok_status();
	}

	static bool IsTransient(iree_hal_buffer_t* BaseBuffer)
	{
		return Cast(BaseBuffer)->RDGBuffer != FRDGBufferRef{};
	}

	static FRDGPooledBuffer* GetPooledBuffer(iree_hal_buffer_t* BaseBuffer)
	{
		if (!BaseBuffer)
		{
			return nullptr;
		}

		iree_hal_buffer_t* AllocatedBuffer = iree_hal_buffer_allocated_buffer(BaseBuffer);

		return Cast(AllocatedBuffer)->PooledBuffer.GetReference();
	}

	static FRDGBufferRef GetRDGBuffer(iree_hal_buffer_t* BaseBuffer, FRDGBuilder* GraphBuilder)
	{
		check(GraphBuilder);

		if (!BaseBuffer)
		{
			return FRDGBufferRef{};
		}

		iree_hal_buffer_t* AllocatedBuffer = iree_hal_buffer_allocated_buffer(BaseBuffer);

		FBuffer* Buffer = Cast(AllocatedBuffer);
		if (Buffer->RDGBuffer)
		{
			check(Buffer->GraphBuilder == GraphBuilder);
			return Buffer->RDGBuffer;
		}
		
		return GraphBuilder->RegisterExternalBuffer(Buffer->PooledBuffer);
	}

private:
	static FBuffer* Cast(const iree_hal_buffer_t* BaseBuffer)
	{
		checkf(iree_hal_resource_is(BaseBuffer, &FBuffer::VTable), TEXT("FBuffer: type does not match"));

		FBuffer* Buffer = (FBuffer*)BaseBuffer;
		check(Buffer->Invariant());

		return Buffer;
	}

	static void Recycle(iree_hal_buffer_t *BaseBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls Buffer 0x%x %ls Size %d", StringCast<TCHAR>(__FUNCTION__).Get(), BaseBuffer, (IsTransient(BaseBuffer) ? TEXT("RDGBuffer") : TEXT("PooledBuffer")), BaseBuffer->allocation_size);
#endif
		iree_hal_buffer_recycle(BaseBuffer);
	}

	static void Destroy(iree_hal_buffer_t *BaseBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls Buffer 0x%x %ls Size %d", StringCast<TCHAR>(__FUNCTION__).Get(), BaseBuffer, (IsTransient(BaseBuffer) ? TEXT("RDGBuffer") : TEXT("PooledBuffer")), BaseBuffer->allocation_size);
#endif
		FBuffer* Buffer = Cast(BaseBuffer);

		if (Buffer->UserReleaseCallback.fn)
		{
			Buffer->UserReleaseCallback.fn(Buffer->UserReleaseCallback.user_data, &Buffer->Base);
		}

		if (IsTransient(BaseBuffer))
		{
			Buffer->RDGBuffer = FRDGBufferRef{};
			Buffer->GraphBuilder = nullptr;
		}
		else
		{
			Buffer->PooledBuffer->Release();
		}

		iree_allocator_free(Buffer->HostAllocator, Buffer);
	}

	static iree_status_t MapRange(iree_hal_buffer_t *BaseBuffer, iree_hal_mapping_mode_t MappingMode, iree_hal_memory_access_t MemoryAccess, iree_device_size_t LocalByteOffset, iree_device_size_t LocalByteLength, iree_hal_buffer_mapping_t* Mapping)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls Buffer 0x%x mm %d ma %d offset %d length %d", StringCast<TCHAR>(__FUNCTION__).Get(), BaseBuffer, (int32)MappingMode, (int32)MemoryAccess, (int32)LocalByteOffset, (int32)LocalByteLength);
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "Memory mapping functionality not available since required GPU-CPU synchronisation not supported by NNERuntimeIREERdg");
	}

	static iree_status_t UnmapRange(iree_hal_buffer_t *BaseBuffer, iree_device_size_t LocalByteOffset, iree_device_size_t LocalByteLength, iree_hal_buffer_mapping_t* Mapping)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls Buffer 0x%x", StringCast<TCHAR>(__FUNCTION__).Get(), BaseBuffer);
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "Memory mapping functionality not available since required GPU-CPU synchronisation not supported by NNERuntimeIREERdg");
	}

	static iree_status_t InvalidateRange(iree_hal_buffer_t *BaseBuffer, iree_device_size_t LocalByteOffset, iree_device_size_t LocalByteLength)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "Memory mapping functionality not available since required GPU-CPU synchronisation not supported by NNERuntimeIREERdg");
	}

	static iree_status_t FlushRange(iree_hal_buffer_t *BaseBuffer, iree_device_size_t LocalByteOffset, iree_device_size_t LocalByteLength)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "Memory mapping functionality not available since required GPU-CPU synchronisation not supported by NNERuntimeIREERdg");
	}

	bool Invariant()
	{
		bool bExclusive = PooledBuffer.IsValid() == (GraphBuilder == nullptr);
		bool bConsistent = (GraphBuilder == nullptr) == (RDGBuffer == nullptr);
		return bExclusive && bConsistent;
	}

	static const iree_hal_buffer_vtable_t VTable;

	iree_hal_buffer_t Base;
	iree_allocator_t HostAllocator;

	TRefCountPtr<FRDGPooledBuffer> PooledBuffer;

	FRDGBuilder* GraphBuilder = nullptr;
	FRDGBufferRef RDGBuffer = nullptr;

	iree_hal_buffer_release_callback_t UserReleaseCallback;
};

const iree_hal_buffer_vtable_t FBuffer::VTable =
{
	.recycle = FBuffer::Recycle,
	.destroy = FBuffer::Destroy,
	.map_range = FBuffer::MapRange,
	.unmap_range = FBuffer::UnmapRange,
	.invalidate_range = FBuffer::InvalidateRange,
	.flush_range = FBuffer::FlushRange
};

} // namespace Private

iree_status_t BufferWrap(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_memory_type_t MemoryType, iree_hal_memory_access_t AllowedAccess, iree_hal_buffer_usage_t AllowedUsage, iree_device_size_t AllocationSize, iree_device_size_t ByteOffset, iree_device_size_t ByteLength, const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer)
{
	return Private::FBuffer::BufferWrap(HostAllocator, DeviceAllocator, MemoryType, AllowedAccess, AllowedUsage, AllocationSize, ByteOffset, ByteLength, PooledBuffer, UserReleaseCallback, OutBuffer);
}

iree_status_t BufferWrapRDG(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_memory_type_t MemoryType, iree_hal_memory_access_t AllowedAccess, iree_hal_buffer_usage_t AllowedUsage, iree_device_size_t AllocationSize, iree_device_size_t ByteOffset, iree_device_size_t ByteLength, FRDGBuilder* GraphBuilder, FRDGBufferRef RDGBuffer, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer)
{
	return Private::FBuffer::BufferWrap(HostAllocator, DeviceAllocator, MemoryType, AllowedAccess, AllowedUsage, AllocationSize, ByteOffset, ByteLength, GraphBuilder, RDGBuffer, UserReleaseCallback, OutBuffer);
}

bool BufferIsTransient(iree_hal_buffer_t* Buffer)
{
	return Private::FBuffer::IsTransient(Buffer);
}

FRDGPooledBuffer* BufferPooledBufferHandle(iree_hal_buffer_t* Buffer)
{
	return Private::FBuffer::GetPooledBuffer(Buffer);
}

FRDGBufferRef BufferRDGBuffer(iree_hal_buffer_t* Buffer, FRDGBuilder* GraphBuilder)
{
	return Private::FBuffer::GetRDGBuffer(Buffer, GraphBuilder);
}

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG