// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDG.h"
#include "Templates/RefCounting.h"

class FRDGPooledBuffer;

namespace UE::IREE::HAL::RDG
{

iree_status_t BufferWrap(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_memory_type_t MemoryType, iree_hal_memory_access_t AllowedAccess, iree_hal_buffer_usage_t AllowedUsage, iree_device_size_t AllocationSize, iree_device_size_t ByteOffset, iree_device_size_t ByteLength, const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer);
bool BufferIsTransient(iree_hal_buffer_t* Buffer);
FRDGPooledBuffer* BufferPooledBufferHandle(iree_hal_buffer_t* Buffer);

} // namespace UE::IREE::HAL::RDG

#endif // WITH_IREE_DRIVER_RDG