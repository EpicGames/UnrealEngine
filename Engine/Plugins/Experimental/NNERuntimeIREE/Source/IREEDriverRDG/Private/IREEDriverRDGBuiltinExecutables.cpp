// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGBuiltinExecutables.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "NNERuntimeIREEShaderFillBufferCS.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

DECLARE_GPU_STAT_NAMED(FDirectCommandBufferFillBuffer, TEXT("DirectCommandBuffer.AddFillBufferPass"));

namespace UE::IREE::HAL::RDG
{

namespace BuiltinExecutables
{

namespace Private
{

template <typename T>
uint32 ExpandToWord(T Pattern);

template<>
uint32 ExpandToWord<uint8>(uint8 Pattern)
{
	return Pattern * 0x01010101;
}

template<>
uint32 ExpandToWord<uint16>(uint16 Pattern)
{
	return Pattern * 0x00010001;
}

uint32 ExpandPatternToWord(uint32 Pattern, uint32 PatternLength)
{
	return PatternLength == 1 ? ExpandToWord<uint8>(static_cast<uint8>(Pattern)):
		PatternLength == 2 ? ExpandToWord<uint16>(static_cast<uint16>(Pattern)) : Pattern;
}

}

iree_status_t FillBuffer(uint8* Buffer, uint32 BufferLength, uint32 Pattern, uint32 PatternLength, uint32 FillOffset, uint32 FillLength)
{
	check(Buffer);
	check(BufferLength >= FillOffset + FillLength);
	check(FillOffset % 4 == 0);
	check(FillLength % 4 == 0);

	const uint32 ExpandedPattern = Private::ExpandPatternToWord(Pattern, PatternLength);
	const uint32 ExpandedLength = 4;

	const uint32 NumRepetitions = FillLength / ExpandedLength;

	for (uint32 i = 0; i < NumRepetitions; i++)
	{
		FPlatformMemory::Memcpy(Buffer + FillOffset + i * ExpandedLength, &ExpandedPattern, ExpandedLength);
	}

	return iree_ok_status();
}

iree_status_t AddFillBufferPass(FRDGBuilder& GraphBuilder, FRDGBufferRef RDGBuffer, uint32 Pattern, uint32 PatternLength, uint32 FillOffset, uint32 FillLength)
{
	SCOPED_NAMED_EVENT_TEXT("BuiltinExecutables::AddFillBufferPass", FColor::Magenta);

	using namespace UE::NNERuntimeIREEShader::Internal;

	if (FillLength == 0)
	{
		return iree_ok_status();
	}

	check(RDGBuffer != FRDGBufferRef{});
	check(RDGBuffer->Desc.GetSize() >= FillOffset + FillLength);
	check(FillOffset % 4 == 0);
	check(FillLength % 4 == 0);

	const uint32 ExpandedPattern = Private::ExpandPatternToWord(Pattern, PatternLength);
	const uint32 ExpandedLength = 4;

	const uint32 NumThreads = FillLength / 4;
	const uint64 NumThreadsPerDispatch = GRHIMaxDispatchThreadGroupsPerDimension.X * FFillBufferConstants::THREAD_GROUP_SIZE;
	const uint32 NumDispatches = (uint32)FMath::DivideAndRoundUp((uint64)NumThreads, NumThreadsPerDispatch);

	for (uint32 i = 0; i < NumDispatches; i++)
	{
		const uint32 DispatchNumThreads = (uint32)FMath::Min((uint64)NumThreads - i * NumThreadsPerDispatch, NumThreadsPerDispatch);
		check(DispatchNumThreads > 0);

		FFillBufferCS::FParameters* ShaderParameters = GraphBuilder.AllocParameters<FFillBufferCS::FParameters>();
		ShaderParameters->TargetBuffer = GraphBuilder.CreateUAV(RDGBuffer);
		ShaderParameters->Fill.X = ExpandedPattern;
		ShaderParameters->Fill.Y = ExpandedLength;
		ShaderParameters->Fill.Z = FillOffset + i * 4 * NumThreadsPerDispatch;
		ShaderParameters->Fill.W = 4 * DispatchNumThreads;

		const int32 ThreadGroupCountX = FMath::DivideAndRoundUp(DispatchNumThreads, FFillBufferConstants::THREAD_GROUP_SIZE);
		const FIntVector GroupCount = FIntVector(ThreadGroupCountX, 1, 1);

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FFillBufferCS> Shader(GlobalShaderMap);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, FDirectCommandBufferFillBuffer, "DirectCommandBuffer.AddFillBufferPass %d with %d Threads", i, ThreadGroupCountX * FFillBufferConstants::THREAD_GROUP_SIZE);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DirectCommandBuffer.AddFillBufferPass"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			Shader,
			ShaderParameters,
			GroupCount);
	}

	return iree_ok_status();
}

} // namespace BuiltinExecutables

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG