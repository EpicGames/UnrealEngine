// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDG.h"
#include "RenderGraphFwd.h"


namespace UE::IREE::HAL::RDG
{

namespace BuiltinExecutables
{

iree_status_t FillBuffer(uint8* Buffer, uint32 BufferLength, uint32 Pattern, uint32 PatternLength, uint32 FillOffset, uint32 FillLength);

iree_status_t AddFillBufferPass(FRDGBuilder& GraphBuilder, FRDGBufferRef RDGBuffer, uint32 Pattern, uint32 PatternLength, uint32 FillOffset, uint32 FillLength);

} // namespace BuiltinExecutables

} // namespace UE::IREE::HAL::RDG

#endif // WITH_IREE_DRIVER_RDG