// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef WITH_IREE_DRIVER_RDG

#if WITH_DEV_AUTOMATION_TESTS
#include "CoreMinimal.h"
#include "IREEDriverRDGBuiltinExecutables.h"
#include "Misc/AutomationTest.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"
#endif

namespace UE::IREE::HAL::RDG
{

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_SHADER_PARAMETER_STRUCT(FFillBufferTestUploadParameters, )
	RDG_BUFFER_ACCESS(TargetBuffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FFillBufferTestDownloadParameters, )
	RDG_BUFFER_ACCESS(TargetBuffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

template <typename ElementType>
void CopyBufferFromCPUToGPU(FRHICommandListImmediate& RHICmdList, const TArray<ElementType>& SrcArray, int32 Count, FRHIBuffer* DstBuffer)
{
	SCOPED_NAMED_EVENT_TEXT("IREEDriverRDG.FillBufferTest.CopyBufferFromCPUToGPU", FColor::Magenta);

	ElementType* Dst = static_cast<ElementType*>(RHICmdList.LockBuffer(DstBuffer, 0, Count * sizeof(ElementType), RLM_WriteOnly));
	const ElementType* Src = SrcArray.GetData();
	
	FPlatformMemory::Memcpy(Dst, Src, Count * sizeof(ElementType));

	RHICmdList.UnlockBuffer(DstBuffer);
}

template <typename ElementType>
void CopyBufferFromGPUToCPU(FRHICommandListImmediate& RHICmdList, FRHIBuffer* SrcBuffer, int32 Count, TArray<ElementType>& DstArray)
{
	const int32 NumBytes = Count * sizeof(ElementType);

	FRHIGPUBufferReadback Readback(TEXT("IREEDriverRDG.FillBufferTest.CopyBufferFromGPUToCPU"));
	Readback.EnqueueCopy(RHICmdList, SrcBuffer, NumBytes);

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	const ElementType* Src = static_cast<ElementType*>(Readback.Lock(NumBytes));
	ElementType* Dst = DstArray.GetData();
	
	FPlatformMemory::Memcpy(Dst, Src, NumBytes);

	Readback.Unlock();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFillBufferMainTest, "IREEDriverRDG.FillBuffer.MainTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags_FeatureMask | EAutomationTestFlags::EngineFilter)
bool FFillBufferMainTest::RunTest(const FString& Parameter)
{
	auto Run = [this] (uint32 BufferSize, uint8 InitialValue, uint32 Pattern, uint32 PatternLength, uint32 FillOffset, uint32 FillLength)
	{
		check(BufferSize % 4 == 0);

		TArray<uint8> ResultCPU;
		ResultCPU.Init(InitialValue, BufferSize);

		BuiltinExecutables::FillBuffer(ResultCPU.GetData(), BufferSize, Pattern, PatternLength, FillOffset, FillLength);

		TArray<uint8> ResultRDG;
		ResultRDG.Init(InitialValue, BufferSize);

		FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

		ENQUEUE_RENDER_COMMAND(NNETransferFunctionTest) ([Signal, &ResultRDG, BufferSize, Pattern, PatternLength, FillOffset, FillLength](FRHICommandListImmediate& RHICmdList) mutable
		{
			ERHIPipeline Pipeline = RHICmdList.GetPipeline();
			if (Pipeline == ERHIPipeline::None)
			{
				RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
			}

			FRDGBuilder	GraphBuilder(RHICmdList);

			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(BufferSize);
			FRDGBufferRef RDGBuffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("IREE::UnitTest::RDGBuffer"));

			{
				FFillBufferTestUploadParameters* Parameters = GraphBuilder.AllocParameters<FFillBufferTestUploadParameters>();
				Parameters->TargetBuffer = RDGBuffer;

				GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.TransferFunctionTest.UploadInput"), Parameters, ERDGPassFlags::Readback,
				[&ResultRDG, BufferSize, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					SCOPED_NAMED_EVENT_TEXT("FTransferFunctionTest::UploadInput", FColor::Magenta);

					CopyBufferFromCPUToGPU(RHICmdList, ResultRDG, BufferSize, Parameters->TargetBuffer->GetRHI());
				});
			}

			BuiltinExecutables::AddFillBufferPass(GraphBuilder, RDGBuffer, Pattern, PatternLength, FillOffset, FillLength);

			{
				FFillBufferTestDownloadParameters* Parameters = GraphBuilder.AllocParameters<FFillBufferTestDownloadParameters>();
				Parameters->TargetBuffer = RDGBuffer;

				GraphBuilder.AddPass(RDG_EVENT_NAME("IREE::UnitTest.FillBuffer.Download"), Parameters, ERDGPassFlags::Readback,
				[&ResultRDG, BufferSize, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					CopyBufferFromGPUToCPU(RHICmdList, Parameters->TargetBuffer->GetRHI(), BufferSize, ResultRDG);
				});
			}

			GraphBuilder.Execute();

			RHICmdList.SubmitAndBlockUntilGPUIdle();

			Signal->Trigger();
		});

		Signal->Wait();

		for (uint32 i = 0; i < BufferSize; i++)
		{
			UTEST_EQUAL(TEXT("Result"), ResultCPU[i], ResultRDG[i]);
		}

		return true;
	};
	// 					BufferSize	InitValue	Pattern		PatternLen	FillOffset	FillLen
	bool bResult = 	Run(64, 		1, 			0, 			4, 			0, 			4);
	bResult &= 		Run(64, 		1, 			0, 			4, 			4, 			8);
	bResult &= 		Run(64, 		1, 			0, 			4, 			60, 		4);

	bResult &= 		Run(64, 		0, 			0x12345678, 4, 			0, 			4);
	bResult &= 		Run(64, 		0, 			0x12345678, 4, 			4, 			4);
	bResult &= 		Run(64, 		0, 			0x12345678, 4, 			60, 		4);

	bResult &= 		Run(64, 		1, 			0, 			1, 			60, 		4);
	bResult &= 		Run(64, 		1, 			0, 			2, 			60, 		4);

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFillBufferExpandPatternTest, "IREEDriverRDG.FillBuffer.ExpandPatternTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags_FeatureMask | EAutomationTestFlags::EngineFilter)
bool FFillBufferExpandPatternTest::RunTest(const FString& Parameter)
{
	constexpr uint32 InitialValue = 1;
	constexpr uint32 BufferSize = 64;
	constexpr uint32 FillOffset = 4;
	constexpr uint32 FillLength = 8;

	{
		TArray<uint8> Result1, Result2;
		Result1.Init(InitialValue, BufferSize);
		Result2.Init(InitialValue, BufferSize);

		BuiltinExecutables::FillBuffer(Result1.GetData(), BufferSize, static_cast<uint32>(0xAB), 1, FillOffset, FillLength);
		BuiltinExecutables::FillBuffer(Result2.GetData(), BufferSize, static_cast<uint32>(0xABAB), 2, FillOffset, FillLength);

		for (uint32 i = 0; i < BufferSize; i++)
		{
			UTEST_EQUAL(TEXT("Result"), Result1[i], Result2[i]);
		}
	}
	{
		TArray<uint8> Result1, Result2;
		Result1.Init(InitialValue, BufferSize);
		Result2.Init(InitialValue, BufferSize);

		BuiltinExecutables::FillBuffer(Result1.GetData(), BufferSize, static_cast<uint32>(0xAB), 1, FillOffset, FillLength);
		BuiltinExecutables::FillBuffer(Result2.GetData(), BufferSize, static_cast<uint32>(0xABABABAB), 4, FillOffset, FillLength);

		for (uint32 i = 0; i < BufferSize; i++)
		{
			UTEST_EQUAL(TEXT("Result"), Result1[i], Result2[i]);
		}
	}
	{
		TArray<uint8> Result1, Result2;
		Result1.Init(InitialValue, BufferSize);
		Result2.Init(InitialValue, BufferSize);

		BuiltinExecutables::FillBuffer(Result1.GetData(), BufferSize, static_cast<uint32>(0xABAB), 2, FillOffset, FillLength);
		BuiltinExecutables::FillBuffer(Result2.GetData(), BufferSize, static_cast<uint32>(0xABABABAB), 4, FillOffset, FillLength);

		for (uint32 i = 0; i < BufferSize; i++)
		{
			UTEST_EQUAL(TEXT("Result"), Result1[i], Result2[i]);
		}
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::IREE::HAL::RDG

#endif // WITH_IREE_DRIVER_RDG
