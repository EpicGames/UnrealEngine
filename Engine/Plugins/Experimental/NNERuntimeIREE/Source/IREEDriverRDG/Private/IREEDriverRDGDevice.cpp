// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGDevice.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "Containers/Map.h"
#include "IREEDriverRDGDeviceAllocator.h"
#include "IREEDriverRDGDirectCommandBuffer.h"
#include "IREEDriverRDGLog.h"
#include "IREEDriverRDGNoOpExecutableCache.h"
#include "IREEDriverRDGSemaphore.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#endif // PLATFORM_MICROSOFT
THIRD_PARTY_INCLUDES_START
#include "iree/base/internal/arena.h"
#include "iree/hal/utils/deferred_command_buffer.h"
#include "iree/hal/utils/file_registry.h"
#include "iree/hal/utils/file_transfer.h"
#include "iree/hal/utils/queue_emulation.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // PLATFORM_MICROSOFT

#ifdef CreateEvent
#undef CreateEvent
#endif

#ifdef CreateSemaphore
#undef CreateSemaphore
#endif

// Note: checkout runtime\src\iree\hal\drivers\vulkan\vulkan_device.cc for implementation variants, specifically for indirect command list.

namespace UE::IREE::HAL::RDG
{

namespace Private
{

class FDevice
{
public:
	static iree_status_t Create(iree_string_view_t Identifier, iree_allocator_t HostAllocator, const TMap<FString, TConstArrayView<uint8>>& Executables, iree_hal_device_t** OutDevice)
	{
		check(OutDevice);

		FDevice* Device = nullptr;
		iree_host_size_t TotalSize = sizeof(*Device) + Identifier.size;

		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, TotalSize, (void**)& Device));
		memset((void*)Device, 0, TotalSize);
		iree_hal_resource_initialize((const void*)&FDevice::VTable, &Device->Resource);

		uint8_t* BufferPtr = (uint8_t*)Device + sizeof(*Device);
		BufferPtr += iree_string_view_append_to_buffer(Identifier, &Device->Identifier, (char*)BufferPtr);

		Device->HostAllocator = HostAllocator;
		iree_status_t Status = DeviceAllocatorCreate(HostAllocator, &Device->DeviceAllocator);
		if (!iree_status_is_ok(Status))
		{
			iree_allocator_free(HostAllocator, Device);
			return Status;
		}

		for (const auto& Pair : Executables)
		{
			Device->Executables.Emplace(Pair.Key, Pair.Value);
		}

		iree_arena_block_pool_initialize(32 * 1024, HostAllocator, &Device->BlockPool);

		*OutDevice = (iree_hal_device_t*)Device;
		return Status;
	}

private:
	static FDevice* Cast(iree_hal_device_t* Device)
	{
		checkf(iree_hal_resource_is(Device, &FDevice::VTable), TEXT("FDevice: type does not match"));
		return (FDevice*)Device;
	}

	static void Destroy(iree_hal_device_t *BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FDevice* Device = Cast(BaseDevice);

		Device->Executables.Empty();

		iree_hal_allocator_release(Device->DeviceAllocator);

		iree_hal_channel_provider_release(Device->ChannelProvider);
		
		// All arena blocks should have been returned.
		iree_arena_block_pool_deinitialize(&Device->BlockPool);
		
		iree_allocator_free(Device->HostAllocator, Device);
	}

	static iree_string_view_t ID(iree_hal_device_t* BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return Cast(BaseDevice)->Identifier;
	}

	static iree_allocator_t GetHostAllocator(iree_hal_device_t* BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return Cast(BaseDevice)->HostAllocator;
	}

	static iree_hal_allocator_t* GetDeviceAllocator(iree_hal_device_t* BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return Cast(BaseDevice)->DeviceAllocator;
	}

	static void SetDeviceAllocator(iree_hal_device_t* BaseDevice, iree_hal_allocator_t* DeviceAllocator)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FDevice* Device = Cast(BaseDevice);
		iree_hal_allocator_retain(DeviceAllocator);
  		iree_hal_allocator_release(Device->DeviceAllocator);
		Device->DeviceAllocator = DeviceAllocator;
	}

	static void SetChannelProvider(iree_hal_device_t* BaseDevice, iree_hal_channel_provider_t* ChannelProvider)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FDevice* Device = Cast(BaseDevice);
		iree_hal_channel_provider_retain(ChannelProvider);
  		iree_hal_channel_provider_release(Device->ChannelProvider);
		Device->ChannelProvider = ChannelProvider;
	}

	static iree_status_t Trim(iree_hal_device_t* BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FDevice* Device = Cast(BaseDevice);
		iree_arena_block_pool_trim(&Device->BlockPool);
		return iree_hal_allocator_trim(Device->DeviceAllocator);
	}

	static iree_status_t QueryInt64(iree_hal_device_t* BaseDevice, iree_string_view_t Category, iree_string_view_t Key, int64_t* OutValue)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls: category='%.*s' key='%.*s'", StringCast<TCHAR>(__FUNCTION__).Get(), (int)Category.size, Category.data, (int)Key.size, Key.data);
#endif
		check(OutValue);

		FDevice* Device = Cast(BaseDevice);
		*OutValue = 0;

		if (iree_string_view_equal(Category, IREE_SV("hal.device.id")))
		{
			*OutValue = iree_string_view_match_pattern(Device->Identifier, Key) ? 1 : 0;
			return iree_ok_status();
		}

		if (iree_string_view_equal(Category, IREE_SV("hal.executable.format")))
		{
			if (iree_string_view_equal(Key, IREE_SV("vulkan-spirv-fb")))
			{
				// Base SPIR-V always supported.
				*OutValue = 1;
				return iree_ok_status();
			}
			if (iree_string_view_equal(Key, IREE_SV("vulkan-spirv-fb-ptr")))
			{
				// SPIR-V with device addresses is optionally supported based on whether
				// we have device feature support.
				// *OutValue = iree_all_bits_set(
				// 				device->logical_device->enabled_features(),
				// 				IREE_HAL_VULKAN_FEATURE_ENABLE_BUFFER_DEVICE_ADDRESSES)
				// 				? 1
				// 				: 0;

				// Currently not supported
				*OutValue = 0;
				return iree_ok_status();
			}
		}

		return iree_make_status(IREE_STATUS_NOT_FOUND, "Unknown device configuration category %.*s :: %.*s", (int)Category.size, Category.data, (int)Key.size, Key.data);
	}

	static iree_status_t QueryCapabilities(iree_hal_device_t* BaseDevice, iree_hal_device_capabilities_t* OutCapabilities)
	{
		memset(OutCapabilities, 0, sizeof(*OutCapabilities));
		return iree_ok_status();
	}

	static const iree_hal_device_topology_info_t* TopologyInfo(iree_hal_device_t* BaseDevice)
	{
		return &Cast(BaseDevice)->TopologyInfoData;
	}

	static iree_status_t RefineTopologyEdge(iree_hal_device_t* SrcDevice, iree_hal_device_t* DstDevice, iree_hal_topology_edge_t* Edge)
	{
		return iree_ok_status();
	}

	static iree_status_t AssignTopologyInfo(iree_hal_device_t* BaseDevice, const iree_hal_device_topology_info_t* TopologyInfo)
	{
		Cast(BaseDevice)->TopologyInfoData = *TopologyInfo;
		return iree_ok_status();
	}

	static iree_status_t CreateChannel(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, iree_hal_channel_params_t Params, iree_hal_channel_t **OutChannel)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "Collectives not implemented");
	}

	static iree_status_t CreateCommandBuffer(iree_hal_device_t* BaseDevice, iree_hal_command_buffer_mode_t Mode, iree_hal_command_category_t CommandCategories, iree_hal_queue_affinity_t QueueAffinity, iree_host_size_t BindingCapacity, iree_hal_command_buffer_t **OutCommandBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif

		FDevice* Device = Cast(BaseDevice);

		if (BindingCapacity > 0)
		{
			return iree_hal_deferred_command_buffer_create(iree_hal_device_allocator(BaseDevice), Mode, CommandCategories, QueueAffinity, BindingCapacity, &Device->BlockPool, iree_hal_device_host_allocator(BaseDevice), OutCommandBuffer);
		}

		return DirectCommandBufferCreate(Device->HostAllocator, Device->DeviceAllocator, Mode, CommandCategories, QueueAffinity, BindingCapacity, OutCommandBuffer);
	}

	static iree_status_t CreateEvent(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, iree_hal_event_flags_t Flags, iree_hal_event_t **OutEvent)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t CreateExecutableCache(iree_hal_device_t* BaseDevice, iree_string_view_t Identifier, iree_loop_t Loop, iree_hal_executable_cache_t **OutExecutableCache)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif

		FDevice* Device = Cast(BaseDevice);
		return NoOpExecutableCacheCreate(Device->HostAllocator, &Device->Executables, OutExecutableCache);
	}

	static iree_status_t ImportFile(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, iree_hal_memory_access_t Access, iree_io_file_handle_t *Handle, iree_hal_external_file_flags_t Flags, iree_hal_file_t **OutFile)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_hal_file_from_handle(iree_hal_device_allocator(BaseDevice), QueueAffinity, Access, Handle, iree_hal_device_host_allocator(BaseDevice), OutFile);
	}

	static iree_status_t CreateSemaphore(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t /*QueueAffinity*/, uint64_t InitialValue, iree_hal_semaphore_flags_t Flags, iree_hal_semaphore_t **OutSemaphore)
	{
		FDevice* Device = Cast(BaseDevice);
		return SemaphoreCreate(Device->HostAllocator, InitialValue, OutSemaphore);
	}

	static iree_hal_semaphore_compatibility_t QuerySemaphoreCompatibility(iree_hal_device_t* BaseDevice, iree_hal_semaphore_t* Semaphore)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return IREE_HAL_SEMAPHORE_COMPATIBILITY_NONE;
	}

	static iree_status_t QueueAlloca(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList, const iree_hal_semaphore_list_t SignalSemaphoreList, iree_hal_allocator_pool_t Pool, iree_hal_buffer_params_t Params, iree_device_size_t AllocationSize, iree_hal_alloca_flags_t Flags, iree_hal_buffer_t **OutBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls AllocationSize %d", StringCast<TCHAR>(__FUNCTION__).Get(), AllocationSize);
#endif
		return iree_hal_allocator_allocate_buffer(iree_hal_device_allocator(BaseDevice), Params, AllocationSize, OutBuffer);
	}

	static iree_status_t QueueDealloca(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList, const iree_hal_semaphore_list_t SignalSemaphoreList, iree_hal_buffer_t *Buffer, iree_hal_dealloca_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls Buffer 0x%x, AllocationSize %d", StringCast<TCHAR>(__FUNCTION__).Get(), Buffer, Buffer->allocation_size);
#endif
		IREE_RETURN_IF_ERROR(iree_hal_device_queue_barrier(BaseDevice, QueueAffinity, WaitSemaphoreList, SignalSemaphoreList, IREE_HAL_EXECUTE_FLAG_NONE));
		return iree_ok_status();
	}

	// static iree_status_t QueueFill(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList,const iree_hal_semaphore_list_t SignalSemaphoreList, iree_hal_buffer_t* TargetBuffer, iree_device_size_t TargetOffset, iree_device_size_t Length, const void* Pattern, iree_host_size_t PatternLength, iree_hal_fill_flags_t Flags)
	// {
// #if IREE_DRIVER_RDG_VERBOSITY == 1
	// 	UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
// #endif
	// 	return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	// }

	// static iree_status_t QueueUpdate(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList, const iree_hal_semaphore_list_t SignalSemaphoreList, const void* SourceBuffer, iree_host_size_t SourceOffset, iree_hal_buffer_t* TargetBuffer, iree_device_size_t TargetOffset, iree_device_size_t Length, iree_hal_update_flags_t Flags)
	// {
// #if IREE_DRIVER_RDG_VERBOSITY == 1
	// 	UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
// #endif
	// 	return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	// }

	// static iree_status_t QueueCopy(iree_hal_device_t* device, iree_hal_queue_affinity_t queue_affinity, const iree_hal_semaphore_list_t wait_semaphore_list, const iree_hal_semaphore_list_t signal_semaphore_list, iree_hal_buffer_t* source_buffer, iree_device_size_t source_offset, iree_hal_buffer_t* target_buffer, iree_device_size_t target_offset, iree_device_size_t length, iree_hal_copy_flags_t flags)
	// {
// #if IREE_DRIVER_RDG_VERBOSITY == 1
	// 	UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
// #endif
	// 	return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	// }

	static iree_status_t QueueRead(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList, const iree_hal_semaphore_list_t SignalSemaphoreList, iree_hal_file_t* SourceFile, uint64_t SourceOffset, iree_hal_buffer_t* TargetBuffer, iree_device_size_t TargetOffset, iree_device_size_t Length, iree_hal_read_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif

		iree_status_t LoopStatus = iree_ok_status();
		iree_hal_file_transfer_options_t Options = {
			/*.loop=*/iree_loop_inline(&LoopStatus),
			/*.chunk_count=*/IREE_HAL_FILE_TRANSFER_CHUNK_COUNT_DEFAULT,
			/*.chunk_size=*/IREE_HAL_FILE_TRANSFER_CHUNK_SIZE_DEFAULT,
		};

		IREE_RETURN_IF_ERROR(iree_hal_device_queue_read_streaming(BaseDevice, QueueAffinity, WaitSemaphoreList, SignalSemaphoreList, SourceFile, SourceOffset, TargetBuffer, TargetOffset, Length, Flags, Options));

		return LoopStatus;
	}

	static iree_status_t QueueWrite(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList, const iree_hal_semaphore_list_t SignalSemaphoreList, iree_hal_buffer_t* SourceBuffer, iree_device_size_t SourceOffset, iree_hal_file_t* TargetFile, uint64_t TargetOffset, iree_device_size_t Length, iree_hal_write_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}
	static iree_status_t QueueExecute(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity, const iree_hal_semaphore_list_t WaitSemaphoreList, const iree_hal_semaphore_list_t SignalSemaphoreList, iree_hal_command_buffer_t* CommandBuffer, iree_hal_buffer_binding_table_t BindingTable, iree_hal_execute_flags_t flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		iree_hal_command_buffer_t* TranslatedCommandBuffer = nullptr;
		iree_status_t Status = iree_ok_status();

		if (CommandBuffer) {
			if (iree_hal_deferred_command_buffer_isa(CommandBuffer))
			{
				Status = CreateCommandBuffer(
					BaseDevice,
					iree_hal_command_buffer_mode(CommandBuffer) | IREE_HAL_COMMAND_BUFFER_MODE_ONE_SHOT |
						// NOTE: we need to validate if a binding table is provided as the
						// bindings were not known when it was originally recorded.
						(iree_hal_buffer_binding_table_is_empty(BindingTable) ? IREE_HAL_COMMAND_BUFFER_MODE_UNVALIDATED : 0),
					iree_hal_command_buffer_allowed_categories(CommandBuffer),
					QueueAffinity,
					/*binding_capacity=*/0,
					&TranslatedCommandBuffer);
				
				if (iree_status_is_ok(Status))
				{
					Status = iree_hal_deferred_command_buffer_apply(CommandBuffer, TranslatedCommandBuffer, BindingTable);
				}
			}
			else
			{
				TranslatedCommandBuffer = CommandBuffer;
				iree_hal_command_buffer_retain(CommandBuffer);
			}
		}

		iree_hal_command_buffer_release(TranslatedCommandBuffer);

		return Status;
	}

	static iree_status_t QueueHostCall(iree_hal_device_t* /*BaseDevice*/, iree_hal_queue_affinity_t /*QueueAffinity*/, const iree_hal_semaphore_list_t /*WaitSemaphoreList*/, const iree_hal_semaphore_list_t /*SignalSemaphoreList*/, iree_hal_host_call_t /*Call*/, const uint64_t /*Args*/[4], iree_hal_host_call_flags_t /*Flags*/)
	{
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "host calls not supported");
	}

	static iree_status_t QueueFlush(iree_hal_device_t* BaseDevice, iree_hal_queue_affinity_t QueueAffinity)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t WaitSemaphore(iree_hal_device_t* BaseDevice, iree_hal_wait_mode_t WaitMode, const iree_hal_semaphore_list_t SemaphoreList, iree_timeout_t Timeout, iree_hal_wait_flags_t /*Flags*/)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t ProfilingBegin(iree_hal_device_t* BaseDevice, const iree_hal_device_profiling_options_t* Options)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t ProfilingFlush(iree_hal_device_t* BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t ProfilingEnd(iree_hal_device_t* BaseDevice)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static const iree_hal_device_vtable_t VTable;

	iree_hal_resource_t Resource;
	iree_string_view_t Identifier;
	iree_allocator_t HostAllocator;
	iree_hal_allocator_t* DeviceAllocator;
	iree_hal_channel_provider_t* ChannelProvider;
	iree_hal_device_topology_info_t TopologyInfoData;
	TMap<FString, TArray<uint8>> Executables;
	// Block pool used for command buffers with a larger block size (as command
	// buffers can contain inlined data uploads).
	iree_arena_block_pool_t BlockPool;
};

const iree_hal_device_vtable_t FDevice::VTable =
{
	.destroy = FDevice::Destroy,
	.id = FDevice::ID,
	.host_allocator = FDevice::GetHostAllocator,
	.device_allocator = FDevice::GetDeviceAllocator,
	.replace_device_allocator = FDevice::SetDeviceAllocator,
	.replace_channel_provider = FDevice::SetChannelProvider,
	.trim = FDevice::Trim,
	.query_i64 = FDevice::QueryInt64,
	.query_capabilities = FDevice::QueryCapabilities,
	.topology_info = FDevice::TopologyInfo,
	.refine_topology_edge = FDevice::RefineTopologyEdge,
	.assign_topology_info = FDevice::AssignTopologyInfo,
	.create_channel = FDevice::CreateChannel,
	.create_command_buffer = FDevice::CreateCommandBuffer,
	.create_event = FDevice::CreateEvent,
	.create_executable_cache = FDevice::CreateExecutableCache,
	.import_file = FDevice::ImportFile,
	.create_semaphore = FDevice::CreateSemaphore,
	.query_semaphore_compatibility = FDevice::QuerySemaphoreCompatibility,
	.queue_alloca = FDevice::QueueAlloca,
	.queue_dealloca = FDevice::QueueDealloca,
	.queue_fill = iree_hal_device_queue_emulated_fill,
	.queue_update = iree_hal_device_queue_emulated_update,
	.queue_copy = iree_hal_device_queue_emulated_copy,
	.queue_read = FDevice::QueueRead,
	.queue_write = FDevice::QueueWrite,
	.queue_host_call = FDevice::QueueHostCall,
	.queue_dispatch = iree_hal_device_queue_emulated_dispatch,
	.queue_execute = FDevice::QueueExecute,
	.queue_flush = FDevice::QueueFlush,
	.wait_semaphores = FDevice::WaitSemaphore,
	.profiling_begin = FDevice::ProfilingBegin,
	.profiling_flush = FDevice::ProfilingFlush,
	.profiling_end = FDevice::ProfilingEnd
};

} // namespace Private

iree_status_t DeviceCreate(iree_string_view_t Identifier, iree_allocator_t HostAllocator, const TMap<FString, TConstArrayView<uint8>>& Executables, iree_hal_device_t** OutDevice)
{
	return Private::FDevice::Create(Identifier, HostAllocator, Executables, OutDevice);
}

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG