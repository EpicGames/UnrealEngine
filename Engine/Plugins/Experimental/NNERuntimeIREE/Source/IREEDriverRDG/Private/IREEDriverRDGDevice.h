// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDG.h"

#endif // WITH_IREE_DRIVER_RDG