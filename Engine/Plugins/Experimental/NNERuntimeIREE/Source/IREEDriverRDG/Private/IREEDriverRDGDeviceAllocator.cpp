// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGDeviceAllocator.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "HAL/Event.h"
#include "IREEDriverRDGBuffer.h"
#include "IREEDriverRDGLog.h"
#include "RenderGraphUtils.h"

namespace UE::IREE::HAL::RDG
{

namespace Private
{

class FDeviceAllocator
{
public:
	static iree_status_t Create(iree_allocator_t HostAllocator, iree_hal_allocator_t** OutDeviceAllocator)
	{
		check(OutDeviceAllocator);

		FDeviceAllocator* DeviceAllocator;
		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, sizeof(*DeviceAllocator), (void**)&DeviceAllocator));
		iree_hal_resource_initialize((const void*)&FDeviceAllocator::VTable, &DeviceAllocator->Resource);

		DeviceAllocator->HostAllocator = HostAllocator;
		DeviceAllocator->Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

		*OutDeviceAllocator = (iree_hal_allocator_t*)DeviceAllocator;
		return iree_ok_status();
	}

	void SetGraphBuilder(FRDGBuilder& InGraphBuilder)
	{
		check(GraphBuilder == nullptr);
		GraphBuilder = &InGraphBuilder;
	}

	FRDGBuilder& GetGraphBuilder() const
	{
		check(GraphBuilder);
		return *GraphBuilder;
	}

	void ResetGraphBuilder()
	{
		check(GraphBuilder);
		GraphBuilder = nullptr;
	}

private:
	static FDeviceAllocator* Cast(const iree_hal_allocator_t* Allocator)
	{
		checkf(iree_hal_resource_is(Allocator, &FDeviceAllocator::VTable), TEXT("FDeviceAllocator: type does not match"));
		return (FDeviceAllocator*)Allocator;
	}

	static void Destroy(iree_hal_allocator_t *BaseAllocator)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FDeviceAllocator* Allocator = Cast(BaseAllocator);

		FGenericPlatformProcess::ReturnSynchEventToPool(Allocator->Signal);
		Allocator->Signal = nullptr;

		Allocator->GraphBuilder = nullptr;
		
		iree_allocator_free(Allocator->HostAllocator, Allocator);
	}

	static iree_allocator_t GetHostAllocator(const iree_hal_allocator_t* BaseAllocator)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
 		return Cast(BaseAllocator)->HostAllocator;
	}

	static iree_status_t Trim(iree_hal_allocator_t* BaseAllocator)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static void QueryStatistics(iree_hal_allocator_t* BaseAllocator, iree_hal_allocator_statistics_t* OutStatistics)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
	}

	static iree_status_t QueryMemoryHeaps(iree_hal_allocator_t* BaseAllocator, iree_host_size_t Capacity, iree_hal_allocator_memory_heap_t* Heaps, iree_host_size_t* OutCount)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_hal_buffer_compatibility_t QueryBufferCompatibility(iree_hal_allocator_t* BaseAllocator, iree_hal_buffer_params_t* Params, iree_device_size_t* AllocationSize)
	{
		check(BaseAllocator);
		check(Params);
		check(AllocationSize);

#if IREE_DRIVER_RDG_VERBOSITY == 1
		iree_bitfield_string_temp_t temp0, temp1, temp2;
		iree_string_view_t memory_type_str = iree_hal_memory_type_format(Params->type, &temp0);
		iree_string_view_t usage_str = iree_hal_buffer_usage_format(Params->usage, &temp1);
		iree_string_view_t access_str = iree_hal_memory_access_format(Params->access, &temp2);

		UE_LOGF(LogIREEDriverRDG, Display, "%ls type %s usage %s access %s allocationSize %lu", StringCast<TCHAR>(__FUNCTION__).Get(), memory_type_str.data, usage_str.data, access_str.data, *AllocationSize);
#endif

		// All buffers can be allocated on the heap.
		iree_hal_buffer_compatibility_t Compatibility = IREE_HAL_BUFFER_COMPATIBILITY_ALLOCATABLE;

		// Buffers can only be used on the queue if they are device visible.
		if (iree_all_bits_set(Params->type, IREE_HAL_MEMORY_TYPE_DEVICE_VISIBLE))
		{
			if (iree_any_bit_set(Params->usage, IREE_HAL_BUFFER_USAGE_TRANSFER))
			{
				Compatibility |= IREE_HAL_BUFFER_COMPATIBILITY_QUEUE_TRANSFER;
			}
			if (iree_any_bit_set(Params->usage, IREE_HAL_BUFFER_USAGE_DISPATCH_STORAGE))
			{
				Compatibility |= IREE_HAL_BUFFER_COMPATIBILITY_QUEUE_DISPATCH;
			}
		}

		if (iree_all_bits_set(Params->type, IREE_HAL_MEMORY_TYPE_DEVICE_LOCAL))
		{
			if (iree_all_bits_set(Params->type, IREE_HAL_MEMORY_TYPE_HOST_VISIBLE))
			{
				UE_LOGF(LogIREEDriverRDG, Display, "Buffer compability for Size %lld: Device local and host visible not supported, falling back to host local and device visible!", (int64)*AllocationSize);
				Params->type &= ~(IREE_HAL_MEMORY_TYPE_DEVICE_LOCAL | IREE_HAL_MEMORY_TYPE_HOST_VISIBLE);
				Params->type |= IREE_HAL_MEMORY_TYPE_HOST_LOCAL | IREE_HAL_MEMORY_TYPE_DEVICE_VISIBLE;
			}
		}

		if (iree_all_bits_set(Params->type, IREE_HAL_MEMORY_TYPE_HOST_LOCAL))
		{
			UE_LOGF(LogIREEDriverRDG, Warning, "Buffer compability for Size %lld: Host local not supported!", (int64)*AllocationSize);
			Compatibility = IREE_HAL_BUFFER_COMPATIBILITY_NONE;
		}

		if (IREE_UNLIKELY(iree_all_bits_set(Params->usage, IREE_HAL_BUFFER_USAGE_MAPPING)))
		{
			UE_LOGF(LogIREEDriverRDG, Display, "Buffer compability for Size %lld contains unsupported IREE_HAL_BUFFER_USAGE_MAPPING bit flag set.", (int64)*AllocationSize);
			Params->usage &= ~IREE_HAL_BUFFER_USAGE_MAPPING;
		}

		// We are now optimal.
		Params->type &= ~IREE_HAL_MEMORY_TYPE_OPTIMAL;

		// Guard against the corner case where the requested buffer size is 0. The
		// application is unlikely to do anything when requesting a 0-byte buffer; but
		// it can happen in real world use cases. So we should at least not crash.
		if (*AllocationSize == 0) *AllocationSize = 4;

		// Align allocation sizes to 4 bytes so shaders operating on 32 bit types can
		// act safely even on buffer ranges that are not naturally aligned.
		*AllocationSize = iree_device_align(*AllocationSize, 4);

		return Compatibility;
	}

	static iree_status_t AllocateBufferInternal(iree_hal_allocator_t* BaseAllocator, const iree_hal_buffer_params_t* Params, iree_device_size_t AllocationSize, iree_hal_buffer_release_callback_t UserReleaseCallback, iree_hal_buffer_t** OutBuffer)
	{
		check(Params);

		FDeviceAllocator* Allocator = Cast(BaseAllocator);

#if IREE_DRIVER_RDG_VERBOSITY == 1
		iree_bitfield_string_temp_t temp0, temp1, temp2;
		iree_string_view_t memory_type_str = iree_hal_memory_type_format(Params->type, &temp0);
		iree_string_view_t usage_str = iree_hal_buffer_usage_format(Params->usage, &temp1);
		iree_string_view_t access_str = iree_hal_memory_access_format(Params->access, &temp2);

		UE_LOGF(LogIREEDriverRDG, Display, "%ls type %s usage %s access %s AllocationSize %d", StringCast<TCHAR>(__FUNCTION__).Get(), memory_type_str.data, usage_str.data, access_str.data, AllocationSize);
#endif

		iree_status_t Status = iree_ok_status();

		if (iree_all_bits_set(Params->type, IREE_HAL_MEMORY_TYPE_DEVICE_LOCAL))
		{
			// Device local case
			check(IsInRenderingThread());

			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(AllocationSize);
			TRefCountPtr<FRDGPooledBuffer> PooledBuffer = AllocatePooledBuffer(BufferDesc, TEXT("IREE::PooledBuffer"));

			Status = BufferWrap(Allocator->HostAllocator, BaseAllocator, Params->type, Params->access, Params->usage, AllocationSize, 0, AllocationSize, PooledBuffer, UserReleaseCallback, OutBuffer);
		}
		else
		{
			// Host local case
			check(iree_all_bits_set(Params->type, IREE_HAL_MEMORY_TYPE_HOST_LOCAL));

#if IREE_DRIVER_RDG_VERBOSITY == 1
			UE_LOGF(LogIREEDriverRDG, Display, "Allocate heap buffer of size %d", AllocationSize);
#endif

			Status = iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
		}

#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "--> Allocator allocated buffer 0x%x", (uint64)*OutBuffer); 
#endif

		return Status;
	}

	static iree_status_t AllocateBuffer(iree_hal_allocator_t* BaseAllocator, const iree_hal_buffer_params_t* Params, iree_device_size_t AllocationSize, iree_hal_buffer_t** OutBuffer)
	{
		check(Params);

#if IREE_DRIVER_RDG_VERBOSITY == 1
		iree_bitfield_string_temp_t temp0, temp1, temp2;
		iree_string_view_t memory_type_str = iree_hal_memory_type_format(Params->type, &temp0);
		iree_string_view_t usage_str = iree_hal_buffer_usage_format(Params->usage, &temp1);
		iree_string_view_t access_str = iree_hal_memory_access_format(Params->access, &temp2);

		UE_LOGF(LogIREEDriverRDG, Display, "%ls %ls type %s usage %s access %s AllocationSize %d", StringCast<TCHAR>(__FUNCTION__).Get(), TEXT("PooledBuffer"), memory_type_str.data, usage_str.data, access_str.data, AllocationSize);
#endif

		// Coerce options into those required by the current device.
		iree_hal_buffer_params_t CompatParams = *Params;
		if (!iree_all_bits_set(QueryBufferCompatibility(BaseAllocator, &CompatParams, &AllocationSize), IREE_HAL_BUFFER_COMPATIBILITY_ALLOCATABLE)) {
			return iree_make_status(IREE_STATUS_INVALID_ARGUMENT, "allocator cannot allocate a buffer with the given parameters");
		}

		return AllocateBufferInternal(BaseAllocator, &CompatParams, AllocationSize, iree_hal_buffer_release_callback_null(), OutBuffer);
	}

	static void DeallocateBuffer(iree_hal_allocator_t* BaseAllocator, iree_hal_buffer_t* Buffer)
	{
		iree_hal_buffer_destroy(Buffer);
	}

	static iree_status_t ImportHostBuffer(iree_hal_allocator_t* BaseAllocator, const iree_hal_buffer_params_t* Params, iree_hal_external_buffer_t* ExternalBuffer, iree_hal_buffer_release_callback_t ReleaseCallback, iree_hal_buffer_t** OutBuffer)
	{
		check(IsInRenderingThread());

#if IREE_DRIVER_RDG_VERBOSITY == 1
		iree_bitfield_string_temp_t temp0, temp1, temp2;
		iree_string_view_t memory_type_str = iree_hal_memory_type_format(Params->type, &temp0);
		iree_string_view_t usage_str = iree_hal_buffer_usage_format(Params->usage, &temp1);
		iree_string_view_t access_str = iree_hal_memory_access_format(Params->access, &temp2);

		UE_LOGF(LogIREEDriverRDG, Display, "%ls type %s usage %s access %s size %d", StringCast<TCHAR>(__FUNCTION__).Get(), memory_type_str.data, usage_str.data, access_str.data, ExternalBuffer->size);
#endif

		FDeviceAllocator* Allocator = Cast(BaseAllocator);

		IREE_RETURN_IF_ERROR(AllocateBufferInternal(BaseAllocator, Params, ExternalBuffer->size, ReleaseCallback, OutBuffer));

		check(Allocator->GraphBuilder);
		FRDGBufferRef RDGBuffer = BufferRDGBuffer(*OutBuffer, Allocator->GraphBuilder);

		Allocator->GraphBuilder->QueueBufferUpload(RDGBuffer, ExternalBuffer->handle.host_allocation.ptr, ExternalBuffer->size, ERDGInitialDataFlags::NoCopy);

		return iree_ok_status();
	}

	static iree_status_t ImportBuffer(iree_hal_allocator_t* BaseAllocator, const iree_hal_buffer_params_t* Params, iree_hal_external_buffer_t* ExternalBuffer, iree_hal_buffer_release_callback_t ReleaseCallback, iree_hal_buffer_t** OutBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		iree_bitfield_string_temp_t temp0, temp1, temp2;
		iree_string_view_t memory_type_str = iree_hal_memory_type_format(Params->type, &temp0);
		iree_string_view_t usage_str = iree_hal_buffer_usage_format(Params->usage, &temp1);
		iree_string_view_t access_str = iree_hal_memory_access_format(Params->access, &temp2);

		UE_LOGF(LogIREEDriverRDG, Display, "%ls type %s usage %s access %s size %d", StringCast<TCHAR>(__FUNCTION__).Get(), memory_type_str.data, usage_str.data, access_str.data, ExternalBuffer->size);
#endif

		// Coerce options into those required by the current device.
		iree_hal_buffer_params_t CompatParams = *Params;
		iree_device_size_t AllocationSize = ExternalBuffer->size;
		if (!iree_all_bits_set(QueryBufferCompatibility(BaseAllocator, &CompatParams, &AllocationSize), IREE_HAL_BUFFER_COMPATIBILITY_ALLOCATABLE)) {
			return iree_make_status(IREE_STATUS_INVALID_ARGUMENT, "allocator cannot allocate a buffer with the given parameters");
		}

		switch (ExternalBuffer->type)
		{
			case IREE_HAL_EXTERNAL_BUFFER_TYPE_HOST_ALLOCATION:
				return ImportHostBuffer(BaseAllocator, Params, ExternalBuffer, ReleaseCallback, OutBuffer);
			// case IREE_HAL_EXTERNAL_BUFFER_TYPE_DEVICE_ALLOCATION:
			// 	return iree_hal_vulkan_native_allocator_import_device_buffer(BaseAllocator, Params, ExternalBuffer, ReleaseCallback, OutBuffer);
			default:
				return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "external buffer type import not implemented");
		}
	}

	static iree_status_t ExportBuffer(iree_hal_allocator_t* BaseAllocator, iree_hal_buffer_t* Buffer, iree_hal_external_buffer_type_t RequestedType, iree_hal_external_buffer_flags_t RequestedFlags, iree_hal_external_buffer_t* OutExternalBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		// UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}
	
	static const iree_hal_allocator_vtable_t VTable;

	iree_hal_resource_t Resource;
	iree_allocator_t HostAllocator;
	FRDGBuilder* GraphBuilder = nullptr;
	FEvent* Signal = nullptr;
};

const iree_hal_allocator_vtable_t FDeviceAllocator::VTable =
{
	.destroy = FDeviceAllocator::Destroy,
	.host_allocator = FDeviceAllocator::GetHostAllocator,
	.trim = FDeviceAllocator::Trim,
	.query_statistics = FDeviceAllocator::QueryStatistics,
	.query_memory_heaps = FDeviceAllocator::QueryMemoryHeaps,
	.query_buffer_compatibility = FDeviceAllocator::QueryBufferCompatibility,
	.allocate_buffer = FDeviceAllocator::AllocateBuffer,
	.deallocate_buffer = FDeviceAllocator::DeallocateBuffer,
	.import_buffer = FDeviceAllocator::ImportBuffer,
	.export_buffer = FDeviceAllocator::ExportBuffer,
	// Virtual memory management operations (optional).
	// Not supported — all entries set to NULL.
	.supports_virtual_memory = NULL,
	.virtual_memory_query_granularity = NULL,
	.virtual_memory_reserve = NULL,
	.virtual_memory_release = NULL,
	.physical_memory_allocate = NULL,
	.physical_memory_free = NULL,
	.virtual_memory_map = NULL,
	.virtual_memory_unmap = NULL,
	.virtual_memory_protect = NULL,
	.virtual_memory_advise = NULL
};

} // namespace Private

iree_status_t DeviceAllocatorCreate(iree_allocator_t HostAllocator, iree_hal_allocator_t** OutDeviceAllocator)
{
	return Private::FDeviceAllocator::Create(HostAllocator, OutDeviceAllocator);
}

void DeviceAllocatorSetGraphBuilder(iree_hal_allocator_t* DeviceAllocator, FRDGBuilder& GraphBuilder)
{
	((Private::FDeviceAllocator*)DeviceAllocator)->SetGraphBuilder(GraphBuilder);
}

FRDGBuilder& DeviceAllocatorGetGraphBuilder(iree_hal_allocator_t* DeviceAllocator)
{
	return ((Private::FDeviceAllocator*)DeviceAllocator)->GetGraphBuilder();
}

void DeviceAllocatorResetGraphBuilder(iree_hal_allocator_t* DeviceAllocator)
{
	((Private::FDeviceAllocator*)DeviceAllocator)->ResetGraphBuilder();
}

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG