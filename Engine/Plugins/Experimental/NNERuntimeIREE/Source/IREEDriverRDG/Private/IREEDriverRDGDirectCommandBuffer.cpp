// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGDirectCommandBuffer.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDGBuffer.h"
#include "IREEDriverRDGBuiltinExecutables.h"
#include "IREEDriverRDGExecutable.h"
#include "IREEDriverRDGLog.h"
#include "NNERuntimeIREEShader.h"
#include "NNERuntimeIREEShaderFillBufferCS.h"
#include "NNERuntimeIREEShaderShared.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

#include "ShaderParameterMetadataBuilder.h"

DECLARE_GPU_STAT_NAMED(FDirectCommandBufferDispatch, TEXT("DirectCommandBuffer.Dispatch"));

namespace UE::IREE::HAL::RDG
{

namespace Private
{

struct FSubspanKey
{
	FRDGBufferRef Buffer = nullptr;

	uint64 Offset = 0;
	uint64 Length = 0;

	FSubspanKey(FRDGBufferRef InBuffer, uint64 InOffset, uint64 InLength) :
		Buffer(InBuffer), Offset(InOffset), Length(InLength)
	{

	}

	bool operator==(const FSubspanKey& Rhs) const
	{
		return Buffer == Rhs.Buffer && Offset == Rhs.Offset && Length == Rhs.Length;
	}
};

inline uint32 GetTypeHash(const FSubspanKey& Key)
{
	return HashCombine(HashCombine(::GetTypeHash(Key.Buffer), ::GetTypeHash(Key.Offset)), ::GetTypeHash(Key.Length));
}

struct FTempEntry
{
	FRDGBufferRef Buffer = nullptr;

	explicit FTempEntry(FRDGBufferRef InBuffer) : Buffer(InBuffer) {}
};

class FDirectCommandBuffer
{
public:
	static iree_status_t Create(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_command_buffer_mode_t Mode, iree_hal_command_category_t CommandCategories, iree_hal_queue_affinity_t QueueAffinity, iree_host_size_t BindingCapacity, iree_hal_command_buffer_t** OutCommandBuffer)
	{
		SCOPED_NAMED_EVENT_TEXT("FDirectCommandBuffer::Create", FColor::Magenta);

		check(OutCommandBuffer);

		FDirectCommandBuffer* CommandBuffer = nullptr;
		iree_host_size_t TotalSize = sizeof(*CommandBuffer) + iree_hal_command_buffer_validation_state_size(Mode, BindingCapacity);

		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, TotalSize, (void**)&CommandBuffer));
		uint8_t* ValidationStatePtr = (uint8_t*)CommandBuffer + sizeof(*CommandBuffer);

		iree_hal_command_buffer_initialize(DeviceAllocator, Mode, CommandCategories, QueueAffinity, BindingCapacity, ValidationStatePtr, &FDirectCommandBuffer::VTable, &CommandBuffer->Base);
		CommandBuffer->HostAllocator = HostAllocator;
		CommandBuffer->DeviceAllocator = DeviceAllocator;

		*OutCommandBuffer = (iree_hal_command_buffer_t*)CommandBuffer;
		return iree_ok_status();
	}

private:
	static FDirectCommandBuffer* Cast(iree_hal_command_buffer_t* CommandBuffer)
	{
		checkf(iree_hal_resource_is(CommandBuffer, &FDirectCommandBuffer::VTable), TEXT("FDirectCommandBuffer: type does not match"));
		return (FDirectCommandBuffer*)CommandBuffer;
	}

	static void Destroy(iree_hal_command_buffer_t* BaseCommandBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FDirectCommandBuffer* CommandBuffer = Cast(BaseCommandBuffer);
		iree_allocator_free(CommandBuffer->HostAllocator, CommandBuffer);
	}

	static iree_status_t Begin(iree_hal_command_buffer_t* BaseCommandBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_ok_status();
	}

	static iree_status_t End(iree_hal_command_buffer_t* BaseCommandBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_ok_status();
	}

	static iree_status_t BeginDebugGroup(iree_hal_command_buffer_t* BaseCommandBuffer, iree_string_view_t Label, iree_hal_label_color_t LabelColor, const iree_hal_label_location_t* Location)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_ok_status();
	}

	static iree_status_t EndDebugGroup(iree_hal_command_buffer_t* BaseCommandBuffer)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_ok_status();
	}

	static iree_status_t ExecutionBarrier(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_execution_stage_t SourceStageMask, iree_hal_execution_stage_t TargetStageMask, iree_hal_execution_barrier_flags_t Flags, iree_host_size_t MemoryBarrierCount, const iree_hal_memory_barrier_t* MemoryBarriers, iree_host_size_t BufferBarrierCount, const iree_hal_buffer_barrier_t* BufferBarriers)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_ok_status();
	}

	static iree_status_t SignalEvent(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_event_t* Event, iree_hal_execution_stage_t SourceStageMask)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t ResetEvent(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_event_t* Event, iree_hal_execution_stage_t SourceStageMask)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t WaitEvents(iree_hal_command_buffer_t* BaseCommandBuffer, iree_host_size_t EventCount, const iree_hal_event_t** Events, iree_hal_execution_stage_t SourceStageMask, iree_hal_execution_stage_t TargetStageMask, iree_host_size_t MemoryBarrierCount, const iree_hal_memory_barrier_t* MemoryBarriers, iree_host_size_t BufferBarrierCount, const iree_hal_buffer_barrier_t* BufferBarriers)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t AdviseBuffer(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_buffer_ref_t BufferRef, iree_hal_memory_advise_flags_t Flags, uint64_t Arg0, uint64_t Arg1)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_ok_status();
	}

	static iree_status_t FillBuffer(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_buffer_ref_t TargetRef, const void* Pattern, iree_device_size_t PatternLength, iree_hal_fill_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		SCOPED_NAMED_EVENT_TEXT("FDirectCommandBuffer::FillBuffer", FColor::Magenta);

		FDirectCommandBuffer* DirectCommandBuffer = Cast(BaseCommandBuffer);

		FRDGBuilder& GraphBuilder = DeviceAllocatorGetGraphBuilder(DirectCommandBuffer->DeviceAllocator);

		FRDGBufferRef RDGBuffer = BufferRDGBuffer(TargetRef.buffer, &GraphBuilder);
		const iree_device_size_t FillOffset = iree_hal_buffer_byte_offset(TargetRef.buffer) + TargetRef.offset;
		const iree_device_size_t BufferLength = iree_hal_buffer_byte_length(TargetRef.buffer);
		const iree_device_size_t FillLength = TargetRef.length == IREE_HAL_WHOLE_BUFFER ? (BufferLength - TargetRef.offset) : TargetRef.length;

		check(TargetRef.offset <= BufferLength);
		check(FillLength <= BufferLength - TargetRef.offset);

		uint32 ShaderPattern = 0;

		switch (PatternLength)
		{
			case 1:
				ShaderPattern = *static_cast<const uint8*>(Pattern);
				break;
			case 2:
				ShaderPattern = *static_cast<const uint16*>(Pattern);
				break;
			case 4:
				ShaderPattern = *static_cast<const uint32*>(Pattern);
				break;
			default:
				return iree_make_status(IREE_STATUS_INVALID_ARGUMENT, "pattern length (%" PRIhsz") is not a power of two or is too large", PatternLength);
		}

		return BuiltinExecutables::AddFillBufferPass(GraphBuilder, RDGBuffer, ShaderPattern, (uint32)PatternLength, (uint32)FillOffset, (uint32)FillLength);
	}

	static iree_status_t UpdateBuffer(iree_hal_command_buffer_t* BaseCommandBuffer, const void* SourceBuffer, iree_host_size_t SourceOffset, iree_hal_buffer_ref_t TargetRef, iree_hal_update_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t CopyBuffer(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_buffer_ref_t SourceRef, iree_hal_buffer_ref_t TargetRef, iree_hal_copy_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls s 0x%x t 0x%x", StringCast<TCHAR>(__FUNCTION__).Get(), (uint64)SourceRef.buffer, (uint64)TargetRef.buffer);
#endif
		SCOPED_NAMED_EVENT_TEXT("FDirectCommandBuffer::CopyBuffer", FColor::Magenta);

		// Host memory not supported
		check(!iree_all_bits_set(SourceRef.buffer->memory_type, IREE_HAL_MEMORY_TYPE_HOST_LOCAL));
		check(!iree_all_bits_set(TargetRef.buffer->memory_type, IREE_HAL_MEMORY_TYPE_HOST_LOCAL));

		FDirectCommandBuffer* DirectCommandBuffer = Cast(BaseCommandBuffer);
		FRDGBuilder& GraphBuilder = DeviceAllocatorGetGraphBuilder(DirectCommandBuffer->DeviceAllocator);

		FRDGBufferRef SourceRDGBuffer = BufferRDGBuffer(SourceRef.buffer, &GraphBuilder);
		FRDGBufferRef TargetRDGBuffer = BufferRDGBuffer(TargetRef.buffer, &GraphBuilder);

		const iree_device_size_t SourceOffset = iree_hal_buffer_byte_offset(SourceRef.buffer) + SourceRef.offset;
		const iree_device_size_t TargetOffset = iree_hal_buffer_byte_offset(TargetRef.buffer) + TargetRef.offset;

		const iree_device_size_t SourceBufferLength = iree_hal_buffer_byte_length(SourceRef.buffer);
		const iree_device_size_t TargetBufferLength = iree_hal_buffer_byte_length(TargetRef.buffer);
		
		const iree_device_size_t ResolvedSourceCopyLength = SourceRef.length == IREE_HAL_WHOLE_BUFFER ? (iree_hal_buffer_byte_length(SourceRef.buffer) - SourceRef.offset) : SourceRef.length;
		const iree_device_size_t ResolvedTargetCopyLength = TargetRef.length == IREE_HAL_WHOLE_BUFFER ? (iree_hal_buffer_byte_length(TargetRef.buffer) - TargetRef.offset) : TargetRef.length;
		check(ResolvedSourceCopyLength == ResolvedTargetCopyLength);
		const iree_device_size_t CopyLength = ResolvedTargetCopyLength;

		check(SourceRef.offset <= SourceBufferLength);
		check(TargetRef.offset <= TargetBufferLength);

		check(CopyLength <= SourceBufferLength - SourceRef.offset);
		check(CopyLength <= TargetBufferLength - TargetRef.offset);

		// RHI does not allow copy with Source == Target
		if (SourceRDGBuffer == TargetRDGBuffer)
		{
			FRDGBufferDesc BufferCopyDesc = FRDGBufferDesc::CreateByteAddressDesc(CopyLength);
			FRDGBufferRef TmpBuffer = GraphBuilder.CreateBuffer(BufferCopyDesc, TEXT("IREE::CopyBufferTmp"));

			AddCopyBufferPass(GraphBuilder, TmpBuffer, 0, SourceRDGBuffer, SourceOffset, CopyLength);
			AddCopyBufferPass(GraphBuilder, TargetRDGBuffer, TargetOffset, TmpBuffer, 0, CopyLength);
		}
		else
		{
			AddCopyBufferPass(GraphBuilder, TargetRDGBuffer, TargetOffset, SourceRDGBuffer, SourceOffset, CopyLength);
		}

		return iree_ok_status();
	}

	static iree_status_t Collective(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_channel_t* Channel, iree_hal_collective_op_t Op, uint32 Param, iree_hal_buffer_ref_t SendingRef, iree_hal_buffer_ref_t ReceivingRef, iree_device_size_t ElementCount)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
	}

	static iree_status_t Dispatch(iree_hal_command_buffer_t* BaseCommandBuffer, iree_hal_executable_t* Executable, iree_hal_executable_export_ordinal_t ExportOrdinal, const iree_hal_dispatch_config_t Config, iree_const_byte_span_t Constants, iree_hal_buffer_ref_list_t Bindings, iree_hal_dispatch_flags_t Flags)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		SCOPED_NAMED_EVENT_TEXT("FDirectCommandBuffer::Dispatch", FColor::Magenta);

		check(BaseCommandBuffer);
		check(Executable);
		
		FDirectCommandBuffer* DirectCommandBuffer = Cast(BaseCommandBuffer);

		FRDGBuilder& GraphBuilder = DeviceAllocatorGetGraphBuilder(DirectCommandBuffer->DeviceAllocator);

		const FNNERuntimeIREEResource* KernelResource;
		IREE_RETURN_IF_ERROR(ExecutableGetResource(Executable, ExportOrdinal, &KernelResource));

		const FString& KernelName = KernelResource->GetFriendlyName();
		const FShaderParametersMetadata* ShaderParameterMetadata = KernelResource->GetShaderParamMetadata();
		const TArray<FShaderParametersMetadata::FMember> &Members = ShaderParameterMetadata->GetMembers();

		FNNERuntimeIREEShader::FParameters *ShaderParameters = GraphBuilder.AllocParameters<FNNERuntimeIREEShader::FParameters>(ShaderParameterMetadata);
		uint8* ShaderParameterDataPtr = (uint8*)ShaderParameters;

		TMap<FSubspanKey, FTempEntry> BufferCopiesToAppend;

		const uint32 StructSize = ShaderParameterMetadata->GetSize();

		uint32 BufferIdx = 0;

		for (int32 i = 0; i < Members.Num(); i++)
		{
			const FShaderParametersMetadata::FMember& Member = Members[i];

			const FString MemberName(Member.GetName());

			EUniformBufferBaseType BaseType = Member.GetBaseType();
			if (Member.IsVariableNativeType())
			{
				SCOPED_NAMED_EVENT_TEXT("Constant", FColor::Magenta);

				check(MemberName == TEXT("Constant"));
				check(Constants.data);
				check(Constants.data_length > 0);
				check(Constants.data_length <= Member.GetMemberSize());
				check(Member.GetOffset() + Constants.data_length <= StructSize);

				uint8* ElementPtr = (uint8*)(ShaderParameterDataPtr + Member.GetOffset());
				FPlatformMemory::Memcpy(ElementPtr, Constants.data, Constants.data_length);
			}
			else
			{
				SCOPED_NAMED_EVENT_TEXT("Buffer", FColor::Magenta);

				check(MemberName.StartsWith(TEXT("Buffer")));
				check(BaseType == EUniformBufferBaseType::UBMT_RDG_BUFFER_UAV);

				const int32 BindingIndex = KernelResource->GetBindingIndex(BufferIdx++);
				check(BindingIndex >= 0);
				check(BindingIndex < Bindings.count);

				const iree_hal_buffer_ref_t& BufferRef = Bindings.values[BindingIndex];
				const uint64 BufferOffset = iree_hal_buffer_byte_offset(BufferRef.buffer) + BufferRef.offset;
				const uint64 BufferRangeLength = iree_hal_buffer_byte_length(BufferRef.buffer);
				const uint64 BufferLength = BufferRef.length == IREE_HAL_WHOLE_BUFFER ? (BufferRangeLength - BufferRef.offset) : (uint64)BufferRef.length;

				FRDGBufferRef RDGBuffer = BufferRDGBuffer(Bindings.values[BindingIndex].buffer, &GraphBuilder);

				check(BufferLength > 0);
				check((BufferOffset & 3ull) == 0 && (BufferLength & 3ull) == 0);
				check(BufferOffset + BufferLength <= RDGBuffer->Desc.GetSize());

				// When we get a buffer range or not a raw buffer we have to create a temp copy to work on and after dispatch copy back
				FRDGBufferRef RDGWorkingBuffer = nullptr;
				if (BufferOffset == 0 && EnumHasAnyFlags(RDGBuffer->Desc.Usage, EBufferUsageFlags::ByteAddressBuffer))
				{
					RDGWorkingBuffer = RDGBuffer;
				}
				else
				{
					const FSubspanKey Key(RDGBuffer, BufferOffset, BufferLength);
					if (auto FindResult = BufferCopiesToAppend.Find(Key))
					{
						RDGWorkingBuffer = FindResult->Buffer;
					}
					else
					{
						FRDGBufferDesc BufferCopyDesc = FRDGBufferDesc::CreateByteAddressDesc(Key.Length);
						FRDGBufferRef BufferCopy = GraphBuilder.CreateBuffer(BufferCopyDesc, TEXT("BufferCopy"));

						AddCopyBufferPass(GraphBuilder, BufferCopy, 0, RDGBuffer, Key.Offset, Key.Length);

						FTempEntry& Entry = BufferCopiesToAppend.Emplace(Key, FTempEntry(BufferCopy));
						RDGWorkingBuffer = Entry.Buffer;
					}
				}

				check(Member.GetOffset() + sizeof(FRDGBufferUAVRef) <= StructSize);

				FRDGBufferUAVRef* ElementPtr = (FRDGBufferUAVRef*)(ShaderParameterDataPtr + Member.GetOffset());
				*ElementPtr = GraphBuilder.CreateUAV(RDGWorkingBuffer);
			}
		}

		TShaderRef<FNNERuntimeIREEShader> Shader = KernelResource->GetShader(0);
		if (Shader.IsValid())
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, FDirectCommandBufferDispatch, "DirectCommandBuffer.Dispatch %s", KernelName);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DirectCommandBuffer.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				Shader,
				ShaderParameterMetadata,
				ShaderParameters,
				FIntVector(Config.workgroup_count[0], Config.workgroup_count[1], Config.workgroup_count[2]));
		}
		else
		{
			UE_LOGF(LogIREEDriverRDG, Warning, "%ls: Missing shader for executable %ls.", StringCast<TCHAR>(__FUNCTION__).Get(), *KernelName);
		}

		for (const auto& Cp : BufferCopiesToAppend)
		{
			AddCopyBufferPass(GraphBuilder, Cp.Key.Buffer, Cp.Key.Offset, Cp.Value.Buffer, 0, Cp.Key.Length);
		}

		return iree_ok_status();
	}

	 static const iree_hal_command_buffer_vtable_t VTable;

	iree_hal_command_buffer_t Base;
	iree_allocator_t HostAllocator;
	iree_hal_allocator_t* DeviceAllocator;
};

const iree_hal_command_buffer_vtable_t FDirectCommandBuffer::VTable = 
{
	.destroy = FDirectCommandBuffer::Destroy,
	.begin = FDirectCommandBuffer::Begin,
	.end = FDirectCommandBuffer::End,
	.begin_debug_group = FDirectCommandBuffer::BeginDebugGroup,
	.end_debug_group = FDirectCommandBuffer::EndDebugGroup,
	.execution_barrier = FDirectCommandBuffer::ExecutionBarrier,
	.signal_event = FDirectCommandBuffer::SignalEvent,
	.reset_event = FDirectCommandBuffer::ResetEvent,
	.wait_events = FDirectCommandBuffer::WaitEvents,
	.advise_buffer = FDirectCommandBuffer::AdviseBuffer,
	.fill_buffer = FDirectCommandBuffer::FillBuffer,
	.update_buffer = FDirectCommandBuffer::UpdateBuffer,
	.copy_buffer = FDirectCommandBuffer::CopyBuffer,
	.collective = FDirectCommandBuffer::Collective,
	.dispatch = FDirectCommandBuffer::Dispatch
};

} // namespace Private

iree_status_t DirectCommandBufferCreate(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_command_buffer_mode_t Mode, iree_hal_command_category_t CommandCategories, iree_hal_queue_affinity_t QueueAffinity, iree_host_size_t BindingCapacity, iree_hal_command_buffer_t** OutCommandBuffer)
{
	return Private::FDirectCommandBuffer::Create(HostAllocator, DeviceAllocator, Mode, CommandCategories, QueueAffinity, BindingCapacity, OutCommandBuffer);
}

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG