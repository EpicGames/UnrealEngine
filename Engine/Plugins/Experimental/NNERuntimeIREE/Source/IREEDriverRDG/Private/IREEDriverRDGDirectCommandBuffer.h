// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDG.h"

namespace UE::IREE::HAL::RDG
{

iree_status_t DirectCommandBufferCreate(iree_allocator_t HostAllocator, iree_hal_allocator_t* DeviceAllocator, iree_hal_command_buffer_mode_t Mode, iree_hal_command_category_t CommandCategories, iree_hal_queue_affinity_t QueueAffinity, iree_host_size_t BindingCapacity, iree_hal_command_buffer_t** OutCommandBuffer);

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG