// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGExecutable.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "NNERuntimeIREEShaderShared.h"

namespace UE::IREE::HAL::RDG
{

namespace Private
{

class FExecutable
{
public:
	iree_hal_resource_t Resource;

private:
	iree_allocator_t HostAllocator;
	TArray<TSharedRef<FNNERuntimeIREEResource>> KernelResources;

public:
	FExecutable(iree_allocator_t HostAllocator, TArrayView<TSharedRef<FNNERuntimeIREEResource>> KernelResources)
		: HostAllocator(HostAllocator) , KernelResources(KernelResources)
	{
		iree_hal_resource_initialize((const void*)&FExecutable::VTable, &Resource);
	}

	static iree_status_t Create(iree_allocator_t HostAllocator, TArrayView<TSharedRef<FNNERuntimeIREEResource>> KernelResources, iree_hal_executable_t** OutExecutable)
	{
		SCOPED_NAMED_EVENT_TEXT("FExecutable::Create", FColor::Magenta);

		check(OutExecutable);

		FExecutable* Executable;
		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, sizeof(*Executable), (void**)&Executable));
		
		new (Executable) FExecutable(HostAllocator, KernelResources);

		*OutExecutable = (iree_hal_executable_t*)Executable;
		return iree_ok_status();
	}

	static iree_status_t GetResource(iree_hal_executable_t* BaseExecutable, int32 EntryPoint, const FNNERuntimeIREEResource** OutResource)
	{
		FExecutable* Executable = Cast(BaseExecutable);

		if (EntryPoint < 0 || EntryPoint >= Executable->KernelResources.Num())
		{
			return iree_make_status(IREE_STATUS_OUT_OF_RANGE, "Invalid entry point index %i requested from executable", EntryPoint);
		}

		*OutResource = &(Executable->KernelResources[EntryPoint].Get());

		return iree_ok_status();
	}

private:
	static FExecutable* Cast(iree_hal_executable_t* Executable)
	{
		checkf(iree_hal_resource_is(Executable, &FExecutable::VTable), TEXT("FExecutable: type does not match"));
		return (FExecutable*)Executable;
	}

	static void Destroy(iree_hal_executable_t* BaseExecutable)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FExecutable* Executable = Cast(BaseExecutable);

		iree_allocator_t HostAllocator = Executable->HostAllocator;

		Executable->~FExecutable();

		iree_allocator_free(HostAllocator, Executable);
	}

	static iree_host_size_t ExportCount(iree_hal_executable_t* /*BaseExecutable*/)
	{
		return 0;
	}

	static iree_status_t ExportInfo(iree_hal_executable_t* /*BaseExecutable*/, iree_hal_executable_export_ordinal_t /*ExportOrdinal*/, iree_hal_executable_export_info_t* /*OutInfo*/)
	{
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "reflection not implemented");
	}

	static iree_status_t ExportParameters(iree_hal_executable_t* /*BaseExecutable*/, iree_hal_executable_export_ordinal_t /*ExportOrdinal*/, iree_host_size_t /*Capacity*/, iree_hal_executable_export_parameter_t* /*OutParameters*/)
	{
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "parameter reflection not implemented");
	}

	static iree_status_t LookupExportByName(iree_hal_executable_t* /*BaseExecutable*/, iree_string_view_t /*Name*/, iree_hal_executable_export_ordinal_t* /*OutExportOrdinal*/)
	{
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "reflection not implemented");
	}

	inline static const iree_hal_executable_vtable_t VTable = []
	{
		iree_hal_executable_vtable_t Result =
		{
			.destroy = FExecutable::Destroy,
			.export_count = FExecutable::ExportCount,
			.export_info = FExecutable::ExportInfo,
			.export_parameters = FExecutable::ExportParameters,
			.lookup_export_by_name = FExecutable::LookupExportByName,
		};
		return Result;

	}();
};

} // namespace Private

iree_status_t ExecutableCreate(iree_allocator_t HostAllocator, TArrayView<TSharedRef<FNNERuntimeIREEResource>> KernelResources, iree_hal_executable_t** OutExecutable)
{
	return Private::FExecutable::Create(HostAllocator, KernelResources, OutExecutable);
}

iree_status_t ExecutableGetResource(iree_hal_executable_t* Executable, int32 EntryPoint, const FNNERuntimeIREEResource** OutResource)
{
	return Private::FExecutable::GetResource(Executable, EntryPoint, OutResource);
}

} // UE::IREE::HAL::RDG

#endif // WITH_IREE_DRIVER_RDG