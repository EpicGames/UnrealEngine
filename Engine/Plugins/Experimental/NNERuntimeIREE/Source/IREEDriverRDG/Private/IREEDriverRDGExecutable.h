// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDG.h"
#include "Templates/SharedPointer.h"

class FNNERuntimeIREEResource;

namespace UE::IREE::HAL::RDG
{

iree_status_t ExecutableCreate(iree_allocator_t HostAllocator, TArrayView<TSharedRef<FNNERuntimeIREEResource>> KernelResources, iree_hal_executable_t** OutExecutable);

iree_status_t ExecutableGetResource(iree_hal_executable_t* Executable, int32 EntryPoint, const FNNERuntimeIREEResource** OutResource);

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG