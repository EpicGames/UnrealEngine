// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGModule.h"

#include "Misc/CoreDelegates.h"
#include "IREEDriverRDGLog.h"

DEFINE_LOG_CATEGORY(LogIREEDriverRDG);

void FIREEDriverRDGModule::StartupModule()
{

}

void FIREEDriverRDGModule::ShutdownModule()
{

}

IMPLEMENT_MODULE(FIREEDriverRDGModule, IREEDriverRDG)