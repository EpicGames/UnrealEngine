// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGNoOpExecutableCache.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDGExecutable.h"
#include "IREEDriverRDGLog.h"
#include "Serialization/MemoryReader.h"
#include "Templates/SharedPointer.h"
#include "IREEDriverRDGShaderParametersMetadata.h"
#include "NNERuntimeIREEShaderShared.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#endif // PLATFORM_MICROSOFT
THIRD_PARTY_INCLUDES_START
#include "iree/hal/utils/executable_debug_info.h"
#include "iree/hal/utils/executable_header.h"

// flatcc schemas:
#include "iree/base/internal/flatcc/parsing.h"
#include "iree/schemas/executable_debug_info_reader.h"
#include "iree/schemas/executable_debug_info_verifier.h"
#include "iree/schemas/unreal_executable_def_reader.h"
#include "iree/schemas/unreal_executable_def_verifier.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // PLATFORM_MICROSOFT

namespace UE::IREE::HAL::RDG
{

namespace Private
{

TArray<uint32> GetBufferBindings(const FIREEDriverRDGShaderParametersMetadata& Metadata)
{
	TArray<uint32> Result;
	for (const FIREEDriverRDGShaderParametersMetadataEntry& Entry : Metadata.Entries)
	{
		if (Entry.Type == FIREEDriverRDGUniformBufferBaseType::BUFFER_UAV)
		{
			Result.Add(Entry.Binding);
		}
	}
	return Result;
}

iree_status_t PreloadIREEResources(const TMap<FString, TArray<uint8>>* Executables, TMap<FString, TSharedRef<FNNERuntimeIREEResource>>& OutPreloadedResources)
{
	SCOPED_NAMED_EVENT_TEXT("PreloadIREEResources", FColor::Magenta);

	for (const TPair<FString, TArray<uint8>>& Pair : *Executables)
	{
		const FString& Name = Pair.Key;
		const TArray<uint8>& Data = Pair.Value;

		TArray<TSharedRef<FNNERuntimeIREEResource>> Resources;
		
		FMemoryReaderView Reader(Data, /*bIsPersitent =*/ true);

		FIREEDriverRDGShaderParametersMetadata IREEShaderParametersMetadata{};
		FIREEDriverRDGShaderParametersMetadata::StaticStruct()->SerializeBin(Reader, &IREEShaderParametersMetadata);

		TUniquePtr<FNNERuntimeIREEShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations = MakeUnique<FNNERuntimeIREEShaderParametersMetadataAllocations>();
		FShaderParametersMetadata* ShaderParametersMetadata = HAL::RDG::BuildShaderParametersMetadata(IREEShaderParametersMetadata, *ShaderParameterMetadataAllocations);

		TSharedRef<FNNERuntimeIREEResource> KernelResource = MakeShared<FNNERuntimeIREEResource>();
		KernelResource->SetupResource(
			GMaxRHIFeatureLevel,
			Name,
			FString(),
			FString(),
			FString(),
			MoveTemp(ShaderParameterMetadataAllocations),
			ShaderParametersMetadata,
			FName(),
			GetBufferBindings(IREEShaderParametersMetadata)
		);
		
		if (!KernelResource->SerializeShaderMap(Reader))
		{
			return iree_make_status(IREE_STATUS_NOT_FOUND, "Loaded shader map is not valid/complete.");
		}

		OutPreloadedResources.Add(Name, KernelResource);
	}

	return iree_ok_status();
}

class FNoOpExecutableCache
{
public:
	static iree_status_t Create(iree_allocator_t HostAllocator, const TMap<FString, TArray<uint8>>* Executables, iree_hal_executable_cache_t** OutExecutableCache)
	{
		SCOPED_NAMED_EVENT_TEXT("FNoOpExecutableCache::Create", FColor::Magenta);

		checkf(IsInGameThread(), TEXT("IREE Executable Cache must be created on Game Thread"));

		check(Executables);
		check(OutExecutableCache);

		FNoOpExecutableCache* ExecutableCache;
		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, sizeof(*ExecutableCache), (void**)&ExecutableCache));
		iree_hal_resource_initialize((const void*)&FNoOpExecutableCache::VTable, &ExecutableCache->Resource);
		ExecutableCache->HostAllocator = HostAllocator;
		
		iree_status_t Status = PreloadIREEResources(Executables, ExecutableCache->PreloadedResources);
		if (!iree_status_is_ok(Status))
		{
			iree_allocator_free(ExecutableCache->HostAllocator, ExecutableCache);
			return Status;
		}

		*OutExecutableCache = (iree_hal_executable_cache_t*)ExecutableCache;
		return iree_ok_status();
	}

private:
	static FNoOpExecutableCache* Cast(iree_hal_executable_cache_t* ExecutableCache)
	{
		checkf(iree_hal_resource_is(ExecutableCache, &FNoOpExecutableCache::VTable), TEXT("FNoOpExecutableCache: type does not match"));
		return (FNoOpExecutableCache*)ExecutableCache;
	}

	static void Destroy(iree_hal_executable_cache_t* BaseExecutableCache)
	{		
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FNoOpExecutableCache* ExecutableCache = Cast(BaseExecutableCache);

		ExecutableCache->PreloadedResources.Empty();

		iree_allocator_free(ExecutableCache->HostAllocator, ExecutableCache);
	}

	static iree_status_t InferFormat(iree_hal_executable_cache_t* /*BaseExecutableCache*/, iree_hal_executable_caching_mode_t /*CachingMode*/, iree_const_byte_span_t ExecutableData, iree_host_size_t ExecutableFormatCapacity, char* ExecutableFormat, iree_host_size_t* OutInferredSize)
	{
		// Read the header prefix (with unsafe inference if size is unknown).
		const bool bUnsafeInferSize = (ExecutableData.data_length == 0);
		iree_const_byte_span_t FlatbufferData = iree_const_byte_span_empty();
		IREE_RETURN_IF_ERROR(iree_hal_read_executable_flatbuffer_header(
			ExecutableData, bUnsafeInferSize,
			iree_hal_unreal_ExecutableDef_file_identifier, &FlatbufferData));

		// Verify the flatbuffer structure.
		if (!iree_hal_unreal_ExecutableDef_verify_as_root(FlatbufferData.data, FlatbufferData.data_length))
		{
			return iree_make_status(IREE_STATUS_INVALID_ARGUMENT, "failed to verify executable flatbuffer structure");
		}

		// Write the format string.
		iree_string_view_t Format = IREE_SV("vulkan-spirv-fb");
		if (Format.size >= ExecutableFormatCapacity)
		{
			return iree_make_status(IREE_STATUS_OUT_OF_RANGE, "executable format buffer too small");
		}
		memcpy(ExecutableFormat, Format.data, Format.size + /*NUL*/ 1);

		// Return the total size (header + flatbuffer).
		*OutInferredSize = sizeof(iree_flatbuffer_file_header_t) + FlatbufferData.data_length;
		return iree_ok_status();
	}

	static bool CanPrepareFormat(iree_hal_executable_cache_t* BaseExecutableCache, iree_hal_executable_caching_mode_t CachingMode, iree_string_view_t ExecutableFormat) 
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls format: %*.hs", StringCast<TCHAR>(__FUNCTION__).Get(), (int)ExecutableFormat.size, ExecutableFormat.data);
#endif
		if (iree_string_view_equal(ExecutableFormat, iree_make_cstring_view("vulkan-spirv-fb")))
		{
			return true;
		}
		// else if (iree_string_view_equal(ExecutableFormat, iree_make_cstring_view("vulkan-spirv-fb-ptr")))
		// {
		// 	return iree_all_bits_set(executable_cache->logical_device->enabled_features(), IREE_HAL_VULKAN_FEATURE_ENABLE_BUFFER_DEVICE_ADDRESSES);
		// }
		return false;
	}

	static iree_status_t PrepareExecutable(iree_hal_executable_cache_t* BaseExecutableCache, const iree_hal_executable_params_t* ExecutableParams, iree_hal_executable_t** OutExecutable) 
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		if (!FNoOpExecutableCache::CanPrepareFormat(BaseExecutableCache, ExecutableParams->caching_mode, ExecutableParams->executable_format))
		{
			return iree_make_status(IREE_STATUS_NOT_FOUND, "No executable implementation registered for the given executable format '%.*s'", (int)ExecutableParams->executable_format.size, ExecutableParams->executable_format.data);
		}

		FNoOpExecutableCache* ExecutableCache = Cast(BaseExecutableCache);

		// Read and strip the flatbuffer file header (64 bytes) to get the raw flatbuffer data.
		iree_const_byte_span_t FlatbufferData = iree_const_byte_span_empty();
		iree_status_t HeaderStatus = iree_hal_read_executable_flatbuffer_header(
				ExecutableParams->executable_data, /*unsafe_infer_size=*/false,
				iree_hal_unreal_ExecutableDef_file_identifier, &FlatbufferData);
		if (!iree_status_is_ok(HeaderStatus))
		{
			return HeaderStatus;
		}

		iree_hal_unreal_ExecutableDef_table_t ExecutableDef = iree_hal_unreal_ExecutableDef_as_root(FlatbufferData.data);
		iree_hal_unreal_UnrealShaderDef_vec_t UnrealShaderVec = iree_hal_unreal_ExecutableDef_unreal_shaders_get(ExecutableDef);
		iree_host_size_t UnrealShaderCount = iree_hal_unreal_UnrealShaderDef_vec_len(UnrealShaderVec);

		TArray<TSharedRef<FNNERuntimeIREEResource>> KernelResources;
		for (iree_host_size_t i = 0; i < UnrealShaderCount; i++)
		{
			iree_hal_unreal_UnrealShaderDef_table_t UnrealShaderDef = iree_hal_unreal_UnrealShaderDef_vec_at(UnrealShaderVec, i);
			flatbuffers_string_t SourceFileName = iree_hal_unreal_UnrealShaderDef_source_file_name_get(UnrealShaderDef);

			FString ShaderName = FString(UTF8_TO_TCHAR(SourceFileName));
			TSharedRef<FNNERuntimeIREEResource>* FoundResources = ExecutableCache->PreloadedResources.Find(ShaderName);
			if (!FoundResources)
			{
				return iree_make_status(IREE_STATUS_NOT_FOUND, "No preloaded IREE resource found for shader '%s'", SourceFileName);
			}
			KernelResources.Add(*FoundResources);
		}

		return ExecutableCreate(ExecutableCache->HostAllocator, KernelResources, OutExecutable);
	}

	static const iree_hal_executable_cache_vtable_t VTable;

	iree_hal_resource_t Resource;
	iree_allocator_t HostAllocator;
	TMap<FString, TSharedRef<FNNERuntimeIREEResource>> PreloadedResources;
};

const iree_hal_executable_cache_vtable_t FNoOpExecutableCache::VTable = 
{
	.destroy = FNoOpExecutableCache::Destroy,
	.infer_format = FNoOpExecutableCache::InferFormat,
	.can_prepare_format = FNoOpExecutableCache::CanPrepareFormat,
	.prepare_executable = FNoOpExecutableCache::PrepareExecutable
};

} // namespace Private

iree_status_t NoOpExecutableCacheCreate(iree_allocator_t HostAllocator, const TMap<FString, TArray<uint8>>* Executables, iree_hal_executable_cache_t** OutExecutableCache)
{
	return Private::FNoOpExecutableCache::Create(HostAllocator, Executables, OutExecutableCache);
}

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG