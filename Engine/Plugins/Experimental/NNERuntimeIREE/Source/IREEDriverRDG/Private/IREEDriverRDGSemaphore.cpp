// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGSemaphore.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "HAL/Platform.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#endif // PLATFORM_MICROSOFT
THIRD_PARTY_INCLUDES_START
#include "IREEDriverRDGLog.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // PLATFORM_MICROSOFT

#include "iree/hal/utils/semaphore_base.h"

namespace UE::IREE::HAL::RDG
{

namespace Private
{

class FSemaphore
{
private:
	iree_hal_semaphore_t Base;
	iree_allocator_t HostAllocator;
public:
	static iree_status_t Create(iree_allocator_t HostAllocator, uint64 InitialValue, iree_hal_semaphore_t** OutSemaphore)
	{
		check(OutSemaphore);

		FSemaphore* Semaphore;
		IREE_RETURN_IF_ERROR(iree_allocator_malloc(HostAllocator, sizeof(*Semaphore), (void**)&Semaphore));
		iree_hal_semaphore_initialize(&FSemaphore::VTable, &Semaphore->Base);
		Semaphore->HostAllocator = HostAllocator;

		*OutSemaphore = (iree_hal_semaphore_t*)Semaphore;
		return iree_ok_status();
	}
private:
	static FSemaphore* Cast(iree_hal_semaphore_t* Semaphore)
	{
		checkf(iree_hal_resource_is(Semaphore, &FSemaphore::VTable), TEXT("FSemaphore: type does not match"));
		return (FSemaphore*)Semaphore;
	}

	static void Destroy(iree_hal_semaphore_t* BaseSemaphore)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		FSemaphore* Semaphore = Cast(BaseSemaphore);
		iree_hal_semaphore_deinitialize(&Semaphore->Base);
		iree_allocator_free(Semaphore->HostAllocator, Semaphore);
	}

	static iree_status_t Query(iree_hal_semaphore_t* BaseSemaphore, uint64_t* OutValue)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		// return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
		return iree_ok_status();
	}

	static iree_status_t Signal(iree_hal_semaphore_t* BaseSemaphore, uint64_t NewValue)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		// return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
		return iree_ok_status();
	}

	static void Fail(iree_hal_semaphore_t* BaseSemaphore, iree_status_t Status)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
	}

	static iree_status_t Wait(iree_hal_semaphore_t* BaseSemaphore, uint64_t Value, iree_timeout_t Timeout, iree_hal_wait_flags_t /*Flags*/)
	{
#if IREE_DRIVER_RDG_VERBOSITY == 1
		UE_LOGF(LogIREEDriverRDG, Display, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());
#endif
		//return iree_make_status(IREE_STATUS_UNIMPLEMENTED, __FUNCTION__);
		return iree_ok_status();
	}

	static iree_status_t ImportTimepoint(iree_hal_semaphore_t* /*BaseSemaphore*/, uint64_t /*Value*/, iree_hal_queue_affinity_t /*QueueAffinity*/, iree_hal_external_timepoint_t /*ExternalTimepoint*/)
	{
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "timepoint import is not yet implemented");
	}

	static iree_status_t ExportTimepoint(iree_hal_semaphore_t* /*BaseSemaphore*/, uint64_t /*Value*/, iree_hal_queue_affinity_t /*QueueAffinity*/, iree_hal_external_timepoint_type_t /*RequestedType*/, iree_hal_external_timepoint_flags_t /*RequestedFlags*/, iree_hal_external_timepoint_t* IREE_RESTRICT /*OutExternalTimepoint*/)
	{
		return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "timepoint export is not yet implemented");
	}

	static const iree_hal_semaphore_vtable_t VTable;
};

const iree_hal_semaphore_vtable_t FSemaphore::VTable =
{
	.destroy = FSemaphore::Destroy,
	.query = FSemaphore::Query,
	.signal = FSemaphore::Signal,
	.fail = FSemaphore::Fail,
	.wait = FSemaphore::Wait,
	.import_timepoint = FSemaphore::ImportTimepoint,
	.export_timepoint = FSemaphore::ExportTimepoint
};

} // namespace Private

iree_status_t SemaphoreCreate(iree_allocator_t HostAllocator, uint64 InitialValue, iree_hal_semaphore_t** OutSemaphore)
{
	return Private::FSemaphore::Create(HostAllocator, InitialValue, OutSemaphore);
}

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG