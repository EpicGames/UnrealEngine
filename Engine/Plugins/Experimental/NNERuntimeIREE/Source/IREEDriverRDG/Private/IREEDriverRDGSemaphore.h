// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDG.h"

namespace UE::IREE::HAL::RDG
{

iree_status_t SemaphoreCreate(iree_allocator_t HostAllocator, uint64 InitialValue, iree_hal_semaphore_t** OutSemaphore);

} // UE::IREE

#endif // WITH_IREE_DRIVER_RDG