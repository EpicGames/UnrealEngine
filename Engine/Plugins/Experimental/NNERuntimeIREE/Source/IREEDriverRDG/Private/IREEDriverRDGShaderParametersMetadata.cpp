// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEDriverRDGShaderParametersMetadata.h"

#ifdef WITH_IREE_DRIVER_RDG

#include "IREEDriverRDGLog.h"
#include "Misc/FileHelper.h"
#include "ShaderParameterMetadataBuilder.h"

namespace UE::IREE::HAL::RDG
{

#if WITH_EDITOR

FIREEDriverRDGUniformBufferBaseType GetBaseType(const FString& TypeStr)
{
	if (TypeStr == TEXT("PARAM")) return FIREEDriverRDGUniformBufferBaseType::PARAM;
	if (TypeStr == TEXT("PARAM_ARRAY")) return FIREEDriverRDGUniformBufferBaseType::PARAM_ARRAY;
	if (TypeStr == TEXT("BUFFER_UAV")) return FIREEDriverRDGUniformBufferBaseType::BUFFER_UAV;
	unimplemented();
	return FIREEDriverRDGUniformBufferBaseType::INVALID;
}

FIREEDriverRDGUniformBufferElementType GetElementType(const FString& TypeStr)
{
	if (TypeStr == TEXT("NONE")) return FIREEDriverRDGUniformBufferElementType::NONE;
	if (TypeStr == TEXT("UINT32")) return FIREEDriverRDGUniformBufferElementType::UINT32;
	unimplemented();
	return FIREEDriverRDGUniformBufferElementType::NONE;
}

bool BuildIREEShaderParametersMetadata(const FString& Filepath, FIREEDriverRDGShaderParametersMetadata& Metadata)
{
	constexpr int32 NUM_COLUMNS = 7;

	FString Filedata;
	if (!FFileHelper::LoadFileToString(Filedata, *Filepath))
	{
		UE_LOGF(LogIREEDriverRDG, Log, "Could not load file to string: %ls", *Filepath);
		return false;
	}

	if (Filedata.IsEmpty())
	{
		UE_LOGF(LogIREEDriverRDG, Log, "File %ls is empty!", *Filepath);
		return false;
	}

	TArray<FString> Lines;
	Filedata.ParseIntoArray(Lines, TEXT("\n"));

	if (Lines.IsEmpty())
	{
		UE_LOGF(LogIREEDriverRDG, Log, "No file content: %ls", *Filepath);
		return false;
	}

	// Skip header row
	for (int i = 1; i < Lines.Num(); i++)
	{
		TArray<FString> Row;
		Lines[i].ParseIntoArray(Row, TEXT(";"), false);

		if (Row.Num() != NUM_COLUMNS)
		{
			UE_LOGF(LogIREEDriverRDG, Log, "Expected %d elements", NUM_COLUMNS);
			return false;
		}

		FIREEDriverRDGShaderParametersMetadataEntry Entry{};
		Entry.Type = GetBaseType(Row[0]);
		Entry.Name = Row[1];
		Entry.ShaderType = Row[2];
		Entry.Binding = FCString::Atoi(*Row[3]);
		Entry.DescriptorSet = FCString::Atoi(*Row[4]);
		Entry.ElementType = Row[5].IsEmpty() ? FIREEDriverRDGUniformBufferElementType::NONE : GetElementType(Row[5]);
		Entry.NumElements = Row[6].IsEmpty() ? 0 : FCString::Atoi(*Row[6]);
		
		Metadata.Entries.Add(MoveTemp(Entry));
	}

	return true;
}

#endif // WITH_EDITOR

FShaderParametersMetadata* BuildShaderParametersMetadata(const FIREEDriverRDGShaderParametersMetadata& Metadata, FNNERuntimeIREEShaderParametersMetadataAllocations& Allocations)
{
	FShaderParametersMetadataBuilder Builder;
	for (const FIREEDriverRDGShaderParametersMetadataEntry& Entry : Metadata.Entries)
	{
		const FString& Name = Allocations.Names.Add_GetRef(Entry.Name);

		switch(Entry.Type)
		{
			case FIREEDriverRDGUniformBufferBaseType::PARAM_ARRAY:
				switch(Entry.ElementType)
				{
					case FIREEDriverRDGUniformBufferElementType::UINT32:
						Builder.AddParamArray<uint32>(*Name, Entry.NumElements);
					break;
					default:
						unimplemented();
				}
			break;
			case FIREEDriverRDGUniformBufferBaseType::BUFFER_UAV:
				Builder.AddRDGBufferUAV(*Name, *Allocations.Names.Add_GetRef(Entry.ShaderType));
			break;
			default:
				unimplemented();
		}
	}

	FShaderParametersMetadata* ShaderParametersMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("iree_shader_parameter_metadata"));

	Allocations.ShaderParameterMetadatas.Reset(ShaderParametersMetadata);

	return ShaderParametersMetadata;
}

} // namespace UE::IREE::HAL::RDG

#endif // WITH_IREE_DRIVER_RDG