// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IREEDriverRDGShaderParametersMetadata.generated.h"

UENUM()
enum class FIREEDriverRDGUniformBufferBaseType : uint8
{
	INVALID,
	PARAM,
	PARAM_ARRAY,
	BUFFER_UAV
};

UENUM()
enum class FIREEDriverRDGUniformBufferElementType : uint8
{
	NONE,
	UINT32
};

USTRUCT()
struct FIREEDriverRDGShaderParametersMetadataEntry
{
	GENERATED_BODY()

	UPROPERTY()
	FIREEDriverRDGUniformBufferBaseType Type = FIREEDriverRDGUniformBufferBaseType::INVALID;

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString ShaderType;

	UPROPERTY()
	uint32 Binding = 0;
	
	UPROPERTY()
	uint32 DescriptorSet = 0;

	UPROPERTY()
	FIREEDriverRDGUniformBufferElementType ElementType = FIREEDriverRDGUniformBufferElementType::NONE;

	UPROPERTY()
	uint32 NumElements = 0;
};

USTRUCT()
struct FIREEDriverRDGShaderParametersMetadata
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FIREEDriverRDGShaderParametersMetadataEntry> Entries;
};

#ifdef WITH_IREE_DRIVER_RDG

#include "Containers/Array.h"
#include "NNERuntimeIREEShaderMetadataAllocations.h"

class FShaderParametersMetadata;

namespace UE::IREE::HAL::RDG
{

#if WITH_EDITOR

bool BuildIREEShaderParametersMetadata(const FString& Filepath, FIREEDriverRDGShaderParametersMetadata& Metadata);

#endif // WITH_EDITOR

FShaderParametersMetadata* BuildShaderParametersMetadata(const FIREEDriverRDGShaderParametersMetadata& Metadata, FNNERuntimeIREEShaderParametersMetadataAllocations& Allocations);

} // namespace UE::IREE::HAL::RDG

#endif // WITH_IREE_DRIVER_RDG