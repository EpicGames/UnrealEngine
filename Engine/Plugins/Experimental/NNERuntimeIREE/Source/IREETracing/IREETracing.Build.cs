// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class IREETracing : ModuleRules
{
	public IREETracing(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"Engine",
				"TraceLog"
			}
		);
	}
}
