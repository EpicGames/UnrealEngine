// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Not set when used to build IREE runtime externally
#ifndef IREETRACING_API
#define IREETRACING_API
#define OUTSIDE_UE

//Define IREE_TRACING_FEATURES based on the requested features and the one we are implementing
#ifndef IREE_TRACING_FEATURES
#define IREE_TRACING_FEATURES (IREE_TRACING_FEATURES_REQUESTED & 1)
#endif  // !IREE_TRACING_FEATURES

#else
#include "CoreMinimal.h"
#endif

#ifdef __cplusplus
extern "C" {
#endif

IREETRACING_API void iree_tracing_set_thread_name(const char* name);

IREETRACING_API uint32_t iree_tracing_zone_begin(const char* name, const char* file, uint32_t line);
IREETRACING_API uint32_t iree_tracing_zone_begin_dynamic(const char* name, size_t name_length, const char* file, uint32_t line);
IREETRACING_API uint32_t iree_tracing_zone_begin_external(const char* file_name, size_t file_name_length, uint32_t line, const char* function_name, size_t function_name_length, const char* name, size_t name_length);
IREETRACING_API void iree_tracing_zone_end(uint32_t zone_id);

IREETRACING_API void iree_tracing_publish_source_file(const void* filename, size_t filename_length, const void* content, size_t content_length);
IREETRACING_API void iree_tracing_zone_append_text_string_view(uint32_t zone_id, const char* txt, size_t size);

#ifdef __cplusplus
}
#endif

#ifdef OUTSIDE_UE
#if IREE_TRACING_FEATURES != 0

#include <stddef.h>
#include <stdint.h>
#include <string.h>

typedef uint32_t iree_zone_id_t;

#define IREE_TRACE(expr) expr

#define IREE_TRACE_SET_THREAD_NAME(name) \
	iree_tracing_set_thread_name(name)

#define IREE_TRACE_ZONE_BEGIN(zone_id) \
	IREE_TRACE_ZONE_BEGIN_NAMED(zone_id, __FUNCTION__)

#define IREE_TRACE_ZONE_BEGIN_NAMED(zone_id, name) \
	iree_zone_id_t zone_id = iree_tracing_zone_begin(name, __FILE__, __LINE__)

#define IREE_TRACE_ZONE_BEGIN_NAMED_DYNAMIC(zone_id, name, name_length) \
	iree_zone_id_t zone_id = iree_tracing_zone_begin_dynamic(name, name_length, __FILE__, __LINE__)

#define IREE_TRACE_ZONE_BEGIN_EXTERNAL(zone_id, file_name, file_name_length, line, function_name, function_name_length, name, name_length) \
	iree_zone_id_t zone_id = iree_tracing_zone_begin_external(file_name, file_name_length, line, function_name, function_name_length, name, name_length)

#define IREE_TRACE_ZONE_END(zone_id) \
	iree_tracing_zone_end(zone_id)

#define IREE_RETURN_AND_END_ZONE_IF_ERROR(zone_id, ...) \
	IREE_RETURN_AND_EVAL_IF_ERROR(IREE_TRACE_ZONE_END(zone_id), __VA_ARGS__)


// Unimplemented
#define IREE_TRACE_PUBLISH_SOURCE_FILE(filename, filename_length, content, content_length) \
	iree_tracing_publish_source_file(filename, filename_length, content, content_length)
#define IREE_TRACE_ZONE_APPEND_TEXT(...) \
	IREE_TRACE_IMPL_GET_VARIADIC_((__VA_ARGS__, IREE_TRACE_ZONE_APPEND_TEXT_STRING_VIEW, IREE_TRACE_ZONE_APPEND_TEXT_CSTRING)) (__VA_ARGS__)
#define IREE_TRACE_ZONE_APPEND_TEXT_CSTRING(zone_id, value) \
	IREE_TRACE_ZONE_APPEND_TEXT_STRING_VIEW(zone_id, value, strlen(value))
#define IREE_TRACE_ZONE_APPEND_TEXT_STRING_VIEW(zone_id, value, value_length) \
	iree_tracing_zone_append_text_string_view(zone_id, value, value_length)
#define IREE_TRACE_ZONE_APPEND_VALUE_I64(zone_id, value)
#define IREE_TRACE_ZONE_SET_COLOR(zone_id, color_xbgr)
#define IREE_TRACE_MESSAGE(level, value_literal)
#define IREE_TRACE_FIBER_ENTER(fiber)
#define IREE_TRACE_FIBER_LEAVE()
#define IREE_TRACE_SET_PLOT_TYPE(name_literal, plot_type, step, fill, color)
#define IREE_TRACE_PLOT_VALUE_I64(name_literal, value)
#define IREE_TRACE_PLOT_VALUE_F32(name_literal, value)
#define IREE_TRACE_PLOT_VALUE_F64(name_literal, value)

// Utilities:
#define IREE_TRACE_IMPL_GET_VARIADIC_HELPER_(_1, _2, _3, NAME, ...) NAME
#define IREE_TRACE_IMPL_GET_VARIADIC_(args) \
	IREE_TRACE_IMPL_GET_VARIADIC_HELPER_ args

#endif // IREE_TRACING_FEATURES & 1
#endif // OUTSIDE_UE