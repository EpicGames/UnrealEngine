// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREETracing.h"

#include "ProfilingDebugging/CpuProfilerTrace.h"

#if !UE_BUILD_SHIPPING && CPUPROFILERTRACE_ENABLED
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformTLS.h"
#include "Trace/Trace.h"

namespace UE::IREETracing::Private
{

// Small helper to produce null-terminated C-strings
template<uint32 NumInlineElements = 256>
struct FAnsiTerminatedSlice
{

	FAnsiTerminatedSlice(const ANSICHAR* In, int32 Len) : Ptr("")
	{
		if (!In || Len <= 0)
		{
			return;
		}

		const int32 Size = FCStringAnsi::Strnlen(In, Len);
		const bool bHasTerminator = (Size < Len);

		if (bHasTerminator)
		{
			Ptr = In;
		}
		else
		{
			Buffer.SetNumUninitialized(Len + 1);
			FMemory::Memcpy(Buffer.GetData(), In, Len);
			Buffer[Len] = '\0';

			Ptr = Buffer.GetData();
		}
	}

	[[nodiscard]] FORCEINLINE const ANSICHAR* Get() const
	{
		return Ptr;
	}

	// Non-copyable to avoid Ptr aliasing a dead Buffer after copying
	FAnsiTerminatedSlice(const FAnsiTerminatedSlice&) = delete;
	FAnsiTerminatedSlice& operator=(const FAnsiTerminatedSlice&) = delete;

	// Moveable
	FAnsiTerminatedSlice(FAnsiTerminatedSlice&&) = default;
	FAnsiTerminatedSlice& operator=(FAnsiTerminatedSlice&&) = default;

private:
	const ANSICHAR* Ptr = nullptr;
	TArray<ANSICHAR, TInlineAllocator<NumInlineElements>> Buffer;
};

} // namespace UE::IREETracing::Private

void iree_tracing_set_thread_name(const char* name)
{
	auto name_conv = StringCast<TCHAR>((const ANSICHAR*)name);

	UE::Trace::ThreadRegister((TCHAR*)name_conv.Get(), FPlatformTLS::GetCurrentThreadId(), -1);
}

uint32_t iree_tracing_zone_begin(const char* name, const char* file, uint32_t line)
{
	uint32_t spec = FCpuProfilerTrace::OutputDynamicEventType(name, file, line);
	FCpuProfilerTrace::OutputBeginEvent(spec);
	return spec;
}

uint32_t iree_tracing_zone_begin_dynamic(const char* name, size_t name_length, const char* file, uint32_t line)
{
	UE::IREETracing::Private::FAnsiTerminatedSlice name_term((const ANSICHAR*)name, name_length);
	auto name_conv = StringCast<TCHAR>(name_term.Get());

	uint32_t spec = FCpuProfilerTrace::OutputDynamicEventType(name_conv.Get(), file, line);
	FCpuProfilerTrace::OutputBeginEvent(spec);
	return spec;
}

uint32_t iree_tracing_zone_begin_external(const char* file_name, size_t file_name_length, uint32_t line, const char* function_name, size_t function_name_length, const char* name, size_t name_length)
{
	UE::IREETracing::Private::FAnsiTerminatedSlice function_name_term((const ANSICHAR*)function_name, function_name_length);
	UE::IREETracing::Private::FAnsiTerminatedSlice file_name_term((const ANSICHAR*)file_name, file_name_length);

	auto function_name_conv = StringCast<ANSICHAR>(function_name_term.Get());
	auto file_name_conv = StringCast<ANSICHAR>(file_name_term.Get());

	return iree_tracing_zone_begin(function_name_conv.Get(), file_name_conv.Get(), line);
}

void iree_tracing_zone_end(uint32_t Spec)
{
	FCpuProfilerTrace::OutputEndEvent();
}

void iree_tracing_publish_source_file(const void* filename, size_t filename_length, const void* content, size_t content_length)
{
	// Unimplemented
}

void iree_tracing_zone_append_text_string_view(uint32_t zone_id, const char* txt, size_t size)
{
	// Unimplemented
}
#else
void iree_tracing_set_thread_name(const char* name) {}
uint32_t iree_tracing_zone_begin(const char* Name, const char* File, uint32_t Line) { return 0; }
uint32_t iree_tracing_zone_begin_dynamic(const char* name, size_t name_length, const char* file, uint32_t line) { return 0; }
uint32_t iree_tracing_zone_begin_external(const char* file_name, size_t file_name_length, uint32_t line, const char* function_name, size_t function_name_length, const char* name, size_t name_length) { return 0; }
void iree_tracing_zone_end(uint32_t Spec) {}
void iree_tracing_publish_source_file(const void* filename, size_t filename_length, const void* content, size_t content_length) {}
void iree_tracing_zone_append_text_string_view(uint32_t zone_id, const char* txt, size_t size) {}
#endif // !UE_BUILD_SHIPPING && CPUPROFILERTRACE_ENABLED