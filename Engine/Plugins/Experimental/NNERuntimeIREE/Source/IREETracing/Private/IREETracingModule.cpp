// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FIREETracingModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};

IMPLEMENT_MODULE(FIREETracingModule, IREETracing)