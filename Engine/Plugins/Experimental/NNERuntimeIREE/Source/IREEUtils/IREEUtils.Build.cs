// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IREEUtils : ModuleRules
{
	public IREEUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core"
			}
		);
	}
}
