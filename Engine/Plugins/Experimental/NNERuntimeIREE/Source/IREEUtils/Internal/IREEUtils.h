// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/SecureHash.h"

namespace UE::IREEUtils
{

IREEUTILS_API bool ResolveSdkPaths(FString& String);

IREEUTILS_API bool ResolveEnvironmentVariables(FString& String);

IREEUTILS_API void RunCommand(const FString& Command, const FString& Arguments, const FString& WorkingDir, const FString& LogFilePath = FString());

IREEUTILS_API bool ImportOnnx(const FString& ImporterCommand, const FString& ImporterArguments, TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TArray64<uint8>& OutMlirData);

/**
* Hashes the contents of the entire file. Depending on the file size this might be a slow operation.
* 
* @return true on success, false otherwise
*/
IREEUTILS_API bool HashAppendFile(FMD5& Hash, const FString& InFilePath);

/**
* Hashes the modification time and the file size.
* 
* @return true on success, false otherwise
*/
IREEUTILS_API bool HashAppendFileStat(FMD5& Hash, const FString& InFilePath);

/**
* Hashes the char array of an FString. This isn't guaranteed to produce the same hash for the same string on different platforms.
*/
IREEUTILS_API void HashAppendString(FMD5& Hash, const FString& Data);

}