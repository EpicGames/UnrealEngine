// Copyright Epic Games, Inc. All Rights Reserved.

#include "IREEUtils.h"

#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IREEUtilsLog.h"
#include "Logging/LogVerbosity.h"
#include "Misc/FileHelper.h"
#include "Misc/MonitoredProcess.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Memory/MemoryView.h"

namespace UE::IREEUtils
{

#if PLATFORM_MAC
namespace Private {

TOptional<FString> GetMacSdkPath()
{
	static TOptional<FString> Cached = []() -> TOptional<FString>
	{
		// Runs /usr/bin/xcrun --sdk macosx --show-sdk-path and captures stdout
		FString Output, Error;
		int32 ReturnCode = -1;
		const bool bLaunched = FPlatformProcess::ExecProcess(TEXT("/usr/bin/xcrun"), TEXT("--sdk macosx --show-sdk-path"), &ReturnCode, &Output, &Error);

		Output.TrimStartAndEndInline();

		if (bLaunched && ReturnCode == 0 && !Output.IsEmpty() && IFileManager::Get().DirectoryExists(*Output))
		{
			UE_LOGF(LogIREEUtils, Log, "Found macOS SDK at %ls", *Output);
			
			return Output;
		}

		if (!Output.IsEmpty())
		{
			UE_LOGF(LogIREEUtils, Warning, "xcrun output: %ls", *Output);
		}
		UE_LOGF(LogIREEUtils, Warning, "Failed to get macOS SDK path (launched = %d, code = %d). Error: %ls", (bLaunched ? 1 : 0), ReturnCode, *Error);

		return {};
	}();

	return Cached;
}

} // namepsace Private		
#endif // PLATFORM_MAC

bool ResolveSdkPaths(FString& String)
{
#if PLATFORM_MAC

	const TOptional<FString> MacSdkPath = Private::GetMacSdkPath();

	if (MacSdkPath.IsSet())
	{
		String.ReplaceInline(*FString("${MAC_SDK_ROOT}"), *MacSdkPath.GetValue());
	}
	else
	{
		if (String.Contains(*FString("${MAC_SDK_ROOT}")))
		{
			return false;
		}
	}
#endif // PLATFORM_MAC	
	return true;
}

bool ResolveEnvironmentVariables(FString& String)
{
	const FString StartString = "$ENV{";
	const FString EndString = "}";
	FString ResultString = String;
	int32 StartIndex = ResultString.Find(StartString, ESearchCase::CaseSensitive);
	while (StartIndex != INDEX_NONE)
	{
		StartIndex += StartString.Len();
		int32 EndIndex = ResultString.Find(EndString, ESearchCase::CaseSensitive, ESearchDir::FromStart, StartIndex);
		if (EndIndex > StartIndex)
		{
			FString EnvironmentVariableName = ResultString.Mid(StartIndex, EndIndex - StartIndex);
			FString EnvironmentVariableValue = FPlatformMisc::GetEnvironmentVariable(*EnvironmentVariableName);
			if (EnvironmentVariableValue.IsEmpty())
			{
				return false;
			}
			else
			{
				ResultString.ReplaceInline(*(StartString + EnvironmentVariableName + EndString), *EnvironmentVariableValue, ESearchCase::CaseSensitive);
			}
		}
		else
		{
			return false;
		}
		StartIndex = ResultString.Find(StartString, ESearchCase::CaseSensitive);
	}
	String = ResultString;
	return true;
}

void RunCommand(const FString& Command, const FString& Arguments, const FString& WorkingDir, const FString& LogFilePath)
{
	int32 ReturnCode = 0;
	bool IsCanceled = false;

	FMonitoredProcess Process(Command, Arguments, WorkingDir, true);
	Process.OnCompleted().BindLambda([&ReturnCode] (int32 _ReturnCode) { ReturnCode = _ReturnCode; });
	Process.OnCanceled().BindLambda([&IsCanceled] (){ IsCanceled = true; });

	if (!Process.Launch())
	{
		UE_LOGF(LogIREEUtils, Warning, "Failed to launch subprocess!");
		return;
	}

	while (Process.Update())
	{
		// Poll until process has finished
	}

	if (IsCanceled)
	{
		UE_LOGF(LogIREEUtils, Warning, "Execution of subprocess was canceled!");
	}
	else if (ReturnCode)
	{
		UE_LOGF(LogIREEUtils, Warning, "Subprocess exited with non-zero code %d", ReturnCode);
	}

	if (!LogFilePath.IsEmpty())
	{
		const FString Output = Process.GetFullOutputWithoutDelegate();

		TStringBuilder<256> Builder;
		Builder.Append(Command)
			.Append(TEXT(" "))
			.Append(Arguments)
			.Append(LINE_TERMINATOR LINE_TERMINATOR)
			.Append(Output);

		FFileHelper::SaveStringToFile(Builder, *LogFilePath);

		UE_LOGF(LogIREEUtils, Log, "Saved subprocess output to: %ls", *LogFilePath);
	}
}

bool ImportOnnx(const FString& ImporterCommand, const FString& ImporterArguments, TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TArray64<uint8>& OutMlirData)
{
	SCOPED_NAMED_EVENT_TEXT("IREEUtils::ImportOnnx", FColor::Magenta);

	using namespace Private;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FString InputFilePath = FPaths::Combine(InOutputDir, InModelName) + ".onnx";
	if (!PlatformFile.FileExists(*InputFilePath))
	{
		SCOPED_NAMED_EVENT_TEXT("InputFile", FColor::Magenta);

		if(!FFileHelper::SaveArrayToFile(InFileData, *InputFilePath))
		{
			UE_LOGF(LogIREEUtils, Warning, "IREECompilerRDG failed to save ONNX model \"%ls\"", *InputFilePath);
			return false;
		}
	}

	FString OutputFilePath = FPaths::Combine(InOutputDir, InModelName) + ".mlir";
	FString IntermediateFilePathNoExt = FPaths::Combine(InOutputDir, InModelName);

	FString ImporterArgumentsCopy = ImporterArguments;
	if (!IREEUtils::ResolveEnvironmentVariables(ImporterArgumentsCopy))
	{
		UE_LOGF(LogIREEUtils, Warning, "IREECompilerRDG could not replace environment variables in %ls", *ImporterArgumentsCopy);
		return false;
	}
	ImporterArgumentsCopy.ReplaceInline(*FString("${INPUT_PATH}"), *(FString("\"") + InputFilePath + "\""));
	ImporterArgumentsCopy.ReplaceInline(*FString("${OUTPUT_PATH}"), *(FString("\"") + OutputFilePath + "\""));

	{
		SCOPED_NAMED_EVENT_TEXT("Import", FColor::Magenta);

		IREEUtils::RunCommand(ImporterCommand, ImporterArgumentsCopy, FPaths::RootDir(), IntermediateFilePathNoExt + "_import-log.txt");
	}

	if (!PlatformFile.FileExists(*OutputFilePath))
	{
		UE_LOGF(LogIREEUtils, Warning, "IREECompilerRDG failed to import the model \"%ls\" using the command:", *InputFilePath);
		UE_LOGF(LogIREEUtils, Warning, "\"%ls\" %ls", *ImporterCommand, *ImporterArgumentsCopy);
		return false;
	}

	{
		SCOPED_NAMED_EVENT_TEXT("Load", FColor::Magenta);
		if(!FFileHelper::LoadFileToArray(OutMlirData, *OutputFilePath))
		{
			UE_LOGF(LogIREEUtils, Warning, "IREECompilerRDG failed to load imported model \"%ls\"", *OutputFilePath);
			return false;
		}
	}

	return true;
}

bool HashAppendFile(FMD5& Hash, const FString& InFilePath)
{
	SCOPED_NAMED_EVENT_TEXT(TEXT("IREEUtils::HashAppendFile"), FColor::Magenta);

	bool bSuccess = FFileHelper::LoadFileInBlocks(InFilePath,
		[&Hash](FMemoryView Block)
		{
			Hash.Update((uint8*)Block.GetData(), Block.GetSize());
		});
	return bSuccess;
}

bool HashAppendFileStat(FMD5& Hash, const FString& InFilePath)
{
	FFileStatData Stat = IFileManager::Get().GetStatData(*InFilePath);
	if (Stat.bIsValid)
	{
		int64 Data[2] =
		{
			Stat.ModificationTime.GetTicks(),
			Stat.FileSize
		};
		Hash.Update((uint8*)Data, sizeof(Data));
		return true;
	}
	return false;
}

void HashAppendString(FMD5& Hash, const FString& Data)
{
	Hash.Update((const uint8*)Data.GetCharArray().GetData(), Data.GetCharArray().NumBytes());
}

} // namespace UE::IREEUtils