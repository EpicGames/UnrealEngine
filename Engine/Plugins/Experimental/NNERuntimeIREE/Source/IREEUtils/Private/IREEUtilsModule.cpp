// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "IREEUtilsLog.h"

DEFINE_LOG_CATEGORY(LogIREEUtils);

class FIREEUtilsModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};

IMPLEMENT_MODULE(FIREEUtilsModule, IREEUtils)