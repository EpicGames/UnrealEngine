// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "NNETypes.h"

namespace UE::NNERuntimeIREE
{
	struct NNERUNTIMEIREE_API FFunctionMetaData
	{
		FString Name;
		TArray<UE::NNE::FTensorDesc> InputDescs;
		TArray<UE::NNE::FTensorDesc> OutputDescs;
	};

	class NNERUNTIMEIREE_API FModuleMetaData
	{

	public:
		void Serialize(FArchive& Ar);

		bool ParseFromBuffer(TConstArrayView64<uint8> Buffer);

		TArray<UE::NNERuntimeIREE::FFunctionMetaData> FunctionMetaData;
	};
} // UE::NNERuntimeIREE
