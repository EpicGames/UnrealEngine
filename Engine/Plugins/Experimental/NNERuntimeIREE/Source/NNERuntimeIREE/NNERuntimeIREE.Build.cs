// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class NNERuntimeIREE : ModuleRules
{
	public NNERuntimeIREE(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"IREE",
				"IREEDriverRDG",
				"IREEUtils",
				"NNE",
				"NNEMlirTools",
				"Projects",
				"RenderCore",
				"RHI"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DerivedDataCache",
					"Json",
					"TargetPlatform"
				}
			);
		}

		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE");
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_RDG");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE");
			PrivateDefinitions.Add("NNE_RUNTIME_IREE_USE_COMBINED_LIB_PATH");
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_RDG");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE");
			PrivateDefinitions.Add("NNE_RUNTIME_IREE_USE_COMBINED_LIB_PATH");
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_RDG");
		}
	}
}
