// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREE.h"

#ifdef WITH_NNE_RUNTIME_IREE

#if WITH_EDITOR
#include "Containers/StringConv.h"
#include "HAL/PlatformFileManager.h"
#include "Memory/SharedBuffer.h"
#include "Misc/FileHelper.h"
#include "NNERuntimeIREEMetaData.h"
#endif // WITH_EDITOR

#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/Platform.h"
#include "Interfaces/ITargetPlatform.h"
#include "IREECompilerRDG.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeIREECompiler.h"
#include "NNERuntimeIREELog.h"
#include "NNERuntimeIREEModel.h"
#include "NNERuntimeIREEModelData.h"
#include "NNERuntimeIREESettings.h"
#include "RHIGlobals.h"
#include "RHIStrings.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace UE::NNERuntimeIREE::Private
{
	UNNERuntimeIREECpuSettings* CreateDefaultRuntimeSettingsCPU()
	{
		UNNERuntimeIREECpuSettings* Settings = NewObject<UNNERuntimeIREECpuSettings>();
		return Settings;
	}

	FString GetTargetPlatformName(const ITargetPlatform* TargetPlatform)
	{
		return TargetPlatform ? TargetPlatform->IniPlatformName() : UGameplayStatics::GetPlatformName();
	}

	FString GetBinariesSubdirectory(const FString& PlatformName)
	{
		if (PlatformName.Equals("Windows"))
		{
			return TEXT("Win64");
		}
		else
		{
			return PlatformName;
		}
	}

	FString GetModelDataIdentifier(const FString& RuntimeName, const FGuid& Guid, int32 Version, const FString& FileIdString, const FString& PlatformName, const FString& Architecture)
	{
		return RuntimeName + "-" + Guid.ToString(EGuidFormats::Digits) + "-" + FString::FromInt(Version) + "-" + FileIdString + "-" + PlatformName + (!Architecture.IsEmpty() ? ("-" + Architecture) : "");
	}

	FString GuidToString(const FGuid& Guid)
	{
		return Guid.ToString(EGuidFormats::Digits).ToLower();
	}

	FString GetRuntimeIdAndVersionString(const FGuid& RuntimeGuid, int32 Version)
	{
		return GuidToString(RuntimeGuid) + "-" + FString::FromInt(Version);
	}

	FString GetModelDataDirectoryName(const FString& RuntimeIdAndVersionString, const FString& FileIdString)
	{
		return RuntimeIdAndVersionString + "_" + FileIdString;
	}

	FString GetIntermediateModelDirPath(const FString& PlatformName, const FString& RuntimeName, const FString& ModelName)
	{
		return FPaths::Combine("Intermediate", "Build", GetBinariesSubdirectory(PlatformName), RuntimeName, ModelName);
	}

	FString GetStagedModelDirPath(const FString& PlatformName, const FString& RuntimeName, const FString& ModelName)
	{
		return FPaths::Combine("Binaries", GetBinariesSubdirectory(PlatformName), RuntimeName, ModelName);
	}

	FString GetPackagedModelDirPath(const FString& PlatformName, const FString& RuntimeName, const FString& ModelName)
	{
		return GetStagedModelDirPath(PlatformName, RuntimeName, ModelName);
	}

	FString GetSharedLibDirPath(const FString& PlatformName, const FString& RuntimeName, const FString& ModelName)
	{
#if WITH_EDITOR
	return GetIntermediateModelDirPath(PlatformName, RuntimeName, ModelName);
#else
	return GetPackagedModelDirPath(PlatformName, RuntimeName, ModelName);
#endif // WITH_EDITOR
	}

	FString GetRuntimeSubdir(bool bIsCooking = false)
	{
#if WITH_EDITOR
		if (bIsCooking)
		{
			return TEXT("Cooked");
		}
		else
		{
			return TEXT("Editor");
		}
#else
		check(!bIsCooking);
		return {};
#endif
	}

	TArray<EShaderPlatform> GetShaderPlatforms(const ITargetPlatform* TargetPlatform)
	{
		TArray<EShaderPlatform> ShaderPlatforms;
		if (TargetPlatform)
		{
			TArray<FName> DesiredShaderFormats;
			TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);

			for (const FName& ShaderFormatName : DesiredShaderFormats)
			{
				ShaderPlatforms.Add(ShaderFormatToLegacyShaderPlatform(ShaderFormatName));
			}
		}
		else
		{
			const ERHIFeatureLevel::Type CacheFeatureLevel = GMaxRHIFeatureLevel;
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[CacheFeatureLevel];

			ShaderPlatforms.Add(ShaderPlatform);
		}

		return ShaderPlatforms;
	}

	bool DoesArchitectureMatch(const FString& InArchitecture)
	{
		if (InArchitecture.IsEmpty())
		{
			return true;
		}

#if PLATFORM_CPU_X86_FAMILY
		return InArchitecture.Equals(TEXT("x86_64"));
#elif PLATFORM_CPU_ARM_FAMILY
		return InArchitecture.Equals(TEXT("arm64"));
#else
		return false;
#endif
	}

#if PLATFORM_CPU_X86_FAMILY
	uint32 ParseX86Feature(const FString& InFeature)
	{
		if (InFeature.Equals(TEXT("sse4.2")))
		{
			return ECPUFeatureBits_X86::SSE42;
		}
		if (InFeature.Equals(TEXT("avx")))
		{
			return ECPUFeatureBits_X86::AVX;
		}
		if (InFeature.Equals(TEXT("avx2")))
		{
			return ECPUFeatureBits_X86::AVX2;
		}
		if (InFeature.Equals(TEXT("avx512")))
		{
			return ECPUFeatureBits_X86::AVX512_NOCAVEATS;
		}
		if (InFeature.Equals(TEXT("f16c")))
		{
			return ECPUFeatureBits_X86::F16C;
		}
		if (InFeature.Equals(TEXT("fma")))
		{
			return ECPUFeatureBits_X86::FMA;
		}
		return 0;
	}

	bool DoX86FeaturesMatch(const TConstArrayView<FString>& InFeatures)
	{
		for (const FString& Feature : InFeatures)
		{
			const uint32 FeatureBits = ParseX86Feature(Feature);
			if (FeatureBits)
			{
				if (!FGenericPlatformMisc::CheckFeatureBit_X86(FeatureBits))
				{
					return false;
				}
			}
			else
			{
				UE_LOGF(LogNNERuntimeIREE, Warning, "NNERuntimeIREE is going to ignore unknown x86 feature %ls.", *Feature);
			}
		}
		return true;
	}
#else
	bool DoX86FeaturesMatch(const TArray<FString>& InFeatures)
	{
		return InFeatures.IsEmpty();
	}
#endif

} // UE::NNERuntimeIREE::Private

FGuid UNNERuntimeIREECpu::GUID = FGuid((int32)'I', (int32)'C', (int32)'P', (int32)'U');
int32 UNNERuntimeIREECpu::Version = 0x00000007; // Increment each time iree compiler/runtime is updated and each time FModelData serialization changes

void UNNERuntimeIREECpu::Init(TSharedRef<UE::NNERuntimeIREE::Private::FEnvironment> InEnvironment)
{
	Environment = InEnvironment;
}

FString UNNERuntimeIREECpu::GetRuntimeName() const
{
	return TEXT("NNERuntimeIREECpu");
}

UNNERuntimeIREECpu::ECanCreateModelDataStatus UNNERuntimeIREECpu::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
#if WITH_EDITOR
	// Check model is not > 2GB
	if ((TArray<uint8>::SizeType)FileData.Num() != FileData.Num())
	{
		return ECanCreateModelDataStatus::Fail;
	}


	return (FileType.Compare(TEXT("mlir"), ESearchCase::IgnoreCase) == 0 || FileType.Compare(TEXT("onnx"), ESearchCase::IgnoreCase) == 0) ? ECanCreateModelDataStatus::Ok : ECanCreateModelDataStatus::FailFileIdNotSupported;
#else
	return ECanCreateModelDataStatus::Fail;
#endif // WITH_EDITOR
}

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeIREECpu::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	SCOPED_NAMED_EVENT_TEXT("UNNERuntimeIREECpu::CreateModelData", FColor::Magenta);

#if WITH_EDITOR
	using namespace UE::NNERuntimeIREE;
	using namespace UE::NNERuntimeIREE::Private;
	using namespace UE::NNERuntimeIREE::CPU::Private;

	const FString TargetPlatformName = GetTargetPlatformName(TargetPlatform);
	if (CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu cannot create the model data with id %ls (Filetype: %ls) for platform %ls", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType, *TargetPlatformName);
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const bool bIsCooking = TargetPlatform != nullptr;

	const FString FileIdString = FileId.ToString(EGuidFormats::Digits).ToLower();
	const FString IntermediateDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetIntermediateModelDirPath(TargetPlatformName, GetRuntimeName(), FileIdString)));
	const FString SharedLibraryDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetSharedLibDirPath(TargetPlatformName, GetRuntimeName(), FileIdString)));

	FString IREEModelDataFilePath = FPaths::Combine(IntermediateDirFullPath, FileIdString) + ".ireemodeldata";

	UNNERuntimeIREECpuSettings* Settings = CreateDefaultRuntimeSettingsCPU();
	if (UNNEModelData::DeserializeRuntimeSettings(AdditionalFileData, *Settings) != UNNEModelData::EDeserializeRuntimeSettings::Success)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to deserialize runtime settings");
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	TUniquePtr<UE::NNERuntimeIREE::CPU::FCompiler> Compiler = UE::NNERuntimeIREE::CPU::FCompiler::Make(TargetPlatformName, Settings->CompilerSettings);
	if (!Compiler.IsValid())
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to create a compiler to compile for platform %ls", *TargetPlatformName);
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}
	TArray64<uint8> ResultData;
	FModelData IREEModelData{};

	bool bNeedCompileMlir = true;
	if (PlatformFile.FileExists(*IREEModelDataFilePath) && FFileHelper::LoadFileToArray(ResultData, *IREEModelDataFilePath))
	{
		SCOPED_NAMED_EVENT_TEXT("Validate", FColor::Magenta);

		if (FModelData::IsSameGuidAndVersion(ResultData, GUID, Version) && IREEModelData.Load(ResultData))
		{
			check(FileIdString.Equals(IREEModelData.FileId.ToString(EGuidFormats::Digits).ToLower()));

			bNeedCompileMlir = Compiler->GetHash() != IREEModelData.CompilerHash;
			for (int32 i = 0; i < IREEModelData.CompilerResult.ArchitectureInfos.Num(); i++)
			{
				const FNNERuntimeIREEArchitectureInfoCPU& Info = IREEModelData.CompilerResult.ArchitectureInfos[i];
				const FString SharedLibrarySubDirFullPath = FPaths::Combine(SharedLibraryDirFullPath, Info.RelativeDirPath);

				const FString SharedLibraryFilePath = FPaths::Combine(SharedLibrarySubDirFullPath, Info.SharedLibraryFileName);
				const FString VmfbFilePath = FPaths::Combine(SharedLibrarySubDirFullPath, Info.VmfbFileName);

				bNeedCompileMlir |= !PlatformFile.FileExists(*SharedLibraryFilePath);
				bNeedCompileMlir |= !PlatformFile.FileExists(*VmfbFilePath);
			}
		}
	}
	
	if (bNeedCompileMlir || bIsCooking)
	{
		SCOPED_NAMED_EVENT_TEXT("Compile", FColor::Magenta);

		PlatformFile.DeleteDirectoryRecursively(*IntermediateDirFullPath);
		PlatformFile.CreateDirectoryTree(*IntermediateDirFullPath);

		TArray64<uint8> ImporterOutputData;
		if(FileType.Compare(TEXT("onnx"), ESearchCase::IgnoreCase) == 0)
		{
			if(!Compiler->ImportOnnx(FileData, FileIdString, IntermediateDirFullPath, ImporterOutputData))
			{
				UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to import ONNX model %ls", *FileIdString);
				return TSharedPtr<UE::NNE::FSharedModelData>();
			}
			FileData = ImporterOutputData;
		}

		// NOTE: From this point FileData refers to mlir data in any case

		if (!Compiler->CompileMlir(FileData, FileIdString, IntermediateDirFullPath, IREEModelData.CompilerResult))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to compile model %ls", *FileIdString);
			return TSharedPtr<UE::NNE::FSharedModelData>();
		}

		IREEModelData.GUID = UNNERuntimeIREECpu::GUID;
		IREEModelData.Version = UNNERuntimeIREECpu::Version;
		IREEModelData.FileId = FileId;
		IREEModelData.CompilerHash = Compiler->GetHash();
		if (AdditionalFileData.Contains("IREEModuleMetaData") && !AdditionalFileData["IREEModuleMetaData"].IsEmpty())
		{
			FMemoryReaderView Reader(AdditionalFileData["IREEModuleMetaData"], /*bIsPersitent =*/ true);
			IREEModelData.ModuleMetaData.Serialize(Reader);
		}
		else
		{
			if (!IREEModelData.ModuleMetaData.ParseFromBuffer(FileData))
			{
				UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to parse meta data %ls", *FileIdString);
				return TSharedPtr<UE::NNE::FSharedModelData>();
			}
		}

		if (!IREEModelData.Store(ResultData) || !FFileHelper::SaveArrayToFile(ResultData, *IREEModelDataFilePath))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to write model data to disk %ls", *IREEModelDataFilePath);
			return TSharedPtr<UE::NNE::FSharedModelData>();
		}
	}

	// Only stage when cooking
	if (bIsCooking)
	{
		// Copy files for staging
		FString StagingDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetPackagedModelDirPath(TargetPlatformName, GetRuntimeName(), FileIdString)));
		for (int32 i = 0; i < IREEModelData.CompilerResult.ArchitectureInfos.Num(); i++)
		{
			SCOPED_NAMED_EVENT_TEXT("Copy", FColor::Magenta);

			const FNNERuntimeIREEArchitectureInfoCPU& Info = IREEModelData.CompilerResult.ArchitectureInfos[i];
			const FString SharedLibrarySubDirFullPath = FPaths::Combine(SharedLibraryDirFullPath, Info.RelativeDirPath);
			const FString StagingSubDirFullPath = FPaths::Combine(StagingDirFullPath, Info.RelativeDirPath);
			
			const FString SharedLibraryFilePathSrc = FPaths::Combine(SharedLibrarySubDirFullPath, Info.SharedLibraryFileName);
			const FString VmfbFilePathSrc = FPaths::Combine(SharedLibrarySubDirFullPath, Info.VmfbFileName);

			const FString SharedLibraryFilePathDest = FPaths::Combine(StagingSubDirFullPath, Info.SharedLibraryFileName);
			const FString VmfbFilePathDest = FPaths::Combine(StagingSubDirFullPath, Info.VmfbFileName);

			IFileManager::Get().Copy(*SharedLibraryFilePathDest, *SharedLibraryFilePathSrc);
			IFileManager::Get().Copy(*VmfbFilePathDest, *VmfbFilePathSrc);
		}
	}

	return MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(ResultData)), 0);
#else
	return TSharedPtr<UE::NNE::FSharedModelData>();
#endif // WITH_EDITOR
}

UObject* UNNERuntimeIREECpu::CreateDefaultRuntimeSettings() const
{
	return UE::NNERuntimeIREE::Private::CreateDefaultRuntimeSettingsCPU();
}

FString UNNERuntimeIREECpu::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	// Leave architecture blank as there is only one model data for all architectures of a given platform, only the vmfb and shared lib are different
	FString PlatformName = UE::NNERuntimeIREE::Private::GetTargetPlatformName(TargetPlatform);
	return UE::NNERuntimeIREE::Private::GetModelDataIdentifier(GetRuntimeName(), UNNERuntimeIREECpu::GUID, UNNERuntimeIREECpu::Version, FileId.ToString(EGuidFormats::Digits), PlatformName, "");
}

UNNERuntimeIREECpu::ECanCreateModelCPUStatus UNNERuntimeIREECpu::CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	check(ModelData != nullptr);

	using namespace UE::NNERuntimeIREE::CPU::Private;

	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());
	if (!SharedData.IsValid())
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	if (!FModelData::IsSameGuidAndVersion(SharedData->GetView(), UNNERuntimeIREECpu::GUID, UNNERuntimeIREECpu::Version))
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	return ECanCreateModelCPUStatus::Ok;
}

TSharedPtr<UE::NNE::IModelCPU> UNNERuntimeIREECpu::CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);

	using namespace UE::NNERuntimeIREE;
	using namespace UE::NNERuntimeIREE::Private;
	using namespace UE::NNERuntimeIREE::CPU::Private;

	if (CanCreateModelCPU(ModelData) != ECanCreateModelCPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu cannot create a model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelCPU>();
	}

	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());
	check(SharedData.IsValid());

	FModelData IREEModelData{};
	if(!IREEModelData.Load(SharedData->GetView()))
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to load model data %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelCPU>();
	}

	int32 TargetIndex = -1;
	for (int32 i = 0; i < IREEModelData.CompilerResult.ArchitectureInfos.Num(); i++)
	{
		bool bMatches = true;
		bMatches &= DoesArchitectureMatch(IREEModelData.CompilerResult.ArchitectureInfos[i].Architecture);
		bMatches &= DoX86FeaturesMatch(IREEModelData.CompilerResult.ArchitectureInfos[i].X86Features);

		if (bMatches)
		{
			TargetIndex = i;
			break;
		}
	}
	if (TargetIndex < 0)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to find a matching target for the current hardware");
		return TSharedPtr<UE::NNE::IModelCPU>();
	}

	const FNNERuntimeIREEArchitectureInfoCPU& ArchitectureInfo = IREEModelData.CompilerResult.ArchitectureInfos[TargetIndex];
	const FString FileIdString = IREEModelData.FileId.ToString(EGuidFormats::Digits).ToLower();
	const FString SharedLibraryDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetSharedLibDirPath(UGameplayStatics::GetPlatformName(), GetRuntimeName(), FileIdString)));
	const FString SharedLibrarySubDirFullPath = FPaths::Combine(SharedLibraryDirFullPath, ArchitectureInfo.RelativeDirPath);

	TSharedPtr<UE::NNE::IModelCPU> Model = UE::NNERuntimeIREE::CPU::FModel::Make(*Environment, SharedLibrarySubDirFullPath, ArchitectureInfo.SharedLibraryFileName, ArchitectureInfo.VmfbFileName, ArchitectureInfo.SharedLibraryEntryPointName, IREEModelData.ModuleMetaData);
	if (!Model.IsValid())
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not initialize the model created from model data with id %ls", *FileIdString);
		return TSharedPtr<UE::NNE::IModelCPU>();
	}
	UE_LOGF(LogNNERuntimeIREE, Display, "UNNERuntimeIREECpu successfully loaded target %ls", *ArchitectureInfo.RelativeDirPath);

	return Model;
}

FString UNNERuntimeIREEGpu::GetRuntimeName() const
{
	return TEXT("");
}

UNNERuntimeIREEGpu::ECanCreateModelDataStatus UNNERuntimeIREEGpu::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
#if WITH_EDITOR
	// Check model is not > 2GB
	if ((TArray<uint8>::SizeType)FileData.Num() != FileData.Num())
	{
		return ECanCreateModelDataStatus::Fail;
	}

	return FileType.Compare(TEXT("mlir"), ESearchCase::IgnoreCase) == 0 ? ECanCreateModelDataStatus::Ok : ECanCreateModelDataStatus::FailFileIdNotSupported;
#else
	return ECanCreateModelDataStatus::Fail;
#endif // WITH_EDITOR
}

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeIREEGpu::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	return TSharedPtr<UE::NNE::FSharedModelData>();
}

FString UNNERuntimeIREEGpu::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	FString PlatformName = UE::NNERuntimeIREE::Private::GetTargetPlatformName(TargetPlatform);
	return UE::NNERuntimeIREE::Private::GetModelDataIdentifier(GetRuntimeName(), GetGUID(), UNNERuntimeIREEGpu::GetVersion(), FileId.ToString(EGuidFormats::Digits), PlatformName, "");
}

UNNERuntimeIREEGpu::ECanCreateModelGPUStatus UNNERuntimeIREEGpu::CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	check(ModelData != nullptr);

	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());
	if (!SharedData.IsValid())
	{
		return ECanCreateModelGPUStatus::Fail;
	}

	TConstArrayView<uint8> SharedDataView = SharedData->GetView();
	FGuid Guid = GetGUID();
	int32 Version = GetVersion();
	int32 GuidSize = sizeof(Guid);
	int32 VersionSize = sizeof(Version);
	if (SharedDataView.Num() <= GuidSize + VersionSize)
	{
		return ECanCreateModelGPUStatus::Fail;
	}

	bool bResult = FGenericPlatformMemory::Memcmp(&(SharedDataView[0]), &(Guid), GuidSize) == 0;
	bResult &= FGenericPlatformMemory::Memcmp(&(SharedDataView[GuidSize]), &(Version), VersionSize) == 0;

	return bResult ? ECanCreateModelGPUStatus::Ok : ECanCreateModelGPUStatus::Fail;
}

TSharedPtr<UE::NNE::IModelGPU> UNNERuntimeIREEGpu::CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);

	if (CanCreateModelGPU(ModelData) != ECanCreateModelGPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREEGpu cannot create a model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelGPU>();
	}

	check(ModelData->GetModelData(GetRuntimeName()).IsValid());

	UE::NNE::IModelGPU* IModel = nullptr;
	TConstArrayView<uint8> SharedDataView = ModelData->GetModelData(GetRuntimeName())->GetView();

	return TSharedPtr<UE::NNE::IModelGPU>(IModel);
}

bool UNNERuntimeIREEGpu::IsAvailable() const
{
	return false;
}

FGuid UNNERuntimeIREEGpu::GetGUID() const
{
	return FGuid();
}

int32 UNNERuntimeIREEGpu::GetVersion() const
{
	return 0;
}

FGuid UNNERuntimeIREECuda::GUID = FGuid((int32)'I', (int32)'G', (int32)'C', (int32)'U');
int32 UNNERuntimeIREECuda::Version = 0x00000001;

FString UNNERuntimeIREECuda::GetRuntimeName() const
{
	return TEXT("NNERuntimeIREECuda");
}

bool UNNERuntimeIREECuda::IsAvailable() const
{
	return false;
}

FGuid UNNERuntimeIREECuda::GetGUID() const
{
	return GUID;
}

int32 UNNERuntimeIREECuda::GetVersion() const
{
	return Version;
}

FGuid UNNERuntimeIREEVulkan::GUID = FGuid((int32)'I', (int32)'G', (int32)'V', (int32)'U');
int32 UNNERuntimeIREEVulkan::Version = 0x00000001;

FString UNNERuntimeIREEVulkan::GetRuntimeName() const
{
	return TEXT("NNERuntimeIREEVulkan");
}

bool UNNERuntimeIREEVulkan::IsAvailable() const
{
	return false;
}

FGuid UNNERuntimeIREEVulkan::GetGUID() const
{
	return GUID;
}

int32 UNNERuntimeIREEVulkan::GetVersion() const
{
	return Version;
}

#ifdef WITH_NNE_RUNTIME_IREE_RDG

FGuid UNNERuntimeIREERdg::GUID = FGuid((int32)'I', (int32)'R', (int32)'D', (int32)'G');
int32 UNNERuntimeIREERdg::Version = 0x00000006; // Increment each time iree compiler/runtime is updated and each time FModelData serialization changes

FString UNNERuntimeIREERdg::GetRuntimeName() const
{
	return TEXT("NNERuntimeIREERdg");
}

UNNERuntimeIREERdg::ECanCreateModelDataStatus UNNERuntimeIREERdg::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
#if WITH_EDITOR
	// Check model is not > 2GB
	if ((TArray<uint8>::SizeType)FileData.Num() != FileData.Num())
	{
		return ECanCreateModelDataStatus::Fail;
	}

	return	(FileType.Compare(TEXT("mlir"), ESearchCase::IgnoreCase) == 0 || FileType.Compare(TEXT("onnx"), ESearchCase::IgnoreCase) == 0) ? ECanCreateModelDataStatus::Ok : ECanCreateModelDataStatus::FailFileIdNotSupported;
#else
	return ECanCreateModelDataStatus::Fail;
#endif // WITH_EDITOR
}

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeIREERdg::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	SCOPED_NAMED_EVENT_TEXT("UNNERuntimeIREERdg::CreateModelData", FColor::Magenta);

#if WITH_EDITOR
	using namespace UE::NNERuntimeIREE;
	using namespace UE::NNERuntimeIREE::Private;
	using namespace UE::NNERuntimeIREE::RDG::Private;

	const FString TargetPlatformName = GetTargetPlatformName(TargetPlatform);
	if (CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg cannot create the model data with id %ls (Filetype: %ls) for platform %ls", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType, *TargetPlatformName);
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	const TArray<EShaderPlatform> ShaderPlatforms = GetShaderPlatforms(TargetPlatform);

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const bool bIsCooking = TargetPlatform != nullptr;
	const FString RuntimeSubdir = GetRuntimeSubdir(bIsCooking);

	const FString FileIdString = FileId.ToString(EGuidFormats::Digits).ToLower();
	const FString IntermediateDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetIntermediateModelDirPath(TargetPlatformName, GetRuntimeName(), FileIdString), RuntimeSubdir));
	const FString SharedLibraryDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetSharedLibDirPath(TargetPlatformName, GetRuntimeName(), FileIdString), RuntimeSubdir));

	FString IREEModelDataFilePath = FPaths::Combine(IntermediateDirFullPath, FileIdString) + ".ireemodeldata";

	TUniquePtr<UE::IREE::Compiler::RDG::FCompiler> Compiler = UE::IREE::Compiler::RDG::FCompiler::Make(TargetPlatform);
	if (!Compiler.IsValid())
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to create a compiler to compile for platform %ls", *TargetPlatformName);
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	bool bNeedCompileMlir = true;
	if (PlatformFile.FileExists(*IREEModelDataFilePath))
	{
		SCOPED_NAMED_EVENT_TEXT("Validate", FColor::Magenta);

		FModelDataHeader Header{};
		TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*IREEModelDataFilePath));
		if (FileReader && FModelData::ReadHeaderAndIsSameGuidAndVersion(*FileReader, GUID, Version, Header))
		{
			check(FileIdString.Equals(Header.FileId.ToString(EGuidFormats::Digits).ToLower()));

			bNeedCompileMlir = Compiler->GetHash() != Header.CompilerHash;
			for (EShaderPlatform ShaderPlatform : ShaderPlatforms)
			{
				const FString ShaderPlatformName = LexToString(ShaderPlatform);

				bNeedCompileMlir |= Header.ShaderPlatforms.Find(ShaderPlatformName) == INDEX_NONE;
			}
		}
	}

	TArray64<uint8> ResultData;
	if (bNeedCompileMlir || bIsCooking)
	{
		SCOPED_NAMED_EVENT_TEXT("Compile", FColor::Magenta);

		PlatformFile.DeleteDirectoryRecursively(*IntermediateDirFullPath);
		PlatformFile.CreateDirectoryTree(*IntermediateDirFullPath);

		TArray64<uint8> ImporterOutputData;
		if(FileType.Compare(TEXT("onnx"), ESearchCase::IgnoreCase) == 0)
		{
			if(!Compiler->ImportOnnx(FileData, FileIdString, IntermediateDirFullPath, ImporterOutputData))
			{
				UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to import ONNX model %ls", *FileIdString);
				return TSharedPtr<UE::NNE::FSharedModelData>();
			}
			FileData = ImporterOutputData;
		}

		// NOTE: From this point FileData refers to mlir data in any case

		FModelData IREEModelData{};
		if (!Compiler->CompileMlir(FileData, FileIdString, IntermediateDirFullPath, ShaderPlatforms, IREEModelData.CompilerResult))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to compile model %ls", *FileIdString);
			return TSharedPtr<UE::NNE::FSharedModelData>();
		}

		IREEModelData.Header.GUID = UNNERuntimeIREERdg::GUID;
		IREEModelData.Header.Version = UNNERuntimeIREERdg::Version;
		IREEModelData.Header.FileId = FileId;
		IREEModelData.Header.CompilerHash = Compiler->GetHash();
		for (EShaderPlatform ShaderPlatform : ShaderPlatforms)
		{
			const FString ShaderPlatformName = LexToString(ShaderPlatform);

			IREEModelData.Header.ShaderPlatforms.Add(ShaderPlatformName);
		}
		if (AdditionalFileData.Contains("IREEModuleMetaData") && !AdditionalFileData["IREEModuleMetaData"].IsEmpty())
		{
			FMemoryReaderView Reader(AdditionalFileData["IREEModuleMetaData"]);
			IREEModelData.ModuleMetaData.Serialize(Reader);
		}
		else
		{
			if (!IREEModelData.ModuleMetaData.ParseFromBuffer(FileData))
			{
				UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to parse meta data %ls", *FileIdString);
				return TSharedPtr<UE::NNE::FSharedModelData>();
			}
		}

		if (!IREEModelData.Store(ResultData) || !FFileHelper::SaveArrayToFile(ResultData, *IREEModelDataFilePath))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to write model data to disk %ls", *IREEModelDataFilePath);
			return TSharedPtr<UE::NNE::FSharedModelData>();
		}
	}
	else
	{
		if (!FFileHelper::LoadFileToArray(ResultData, *IREEModelDataFilePath))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to read model data from disk %ls", *IREEModelDataFilePath);
			return TSharedPtr<UE::NNE::FSharedModelData>();
		}
	}

	return MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(ResultData)), 0);
#else
	return TSharedPtr<UE::NNE::FSharedModelData>();
#endif // WITH_EDITOR
}

FString UNNERuntimeIREERdg::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	FString PlatformName = UE::NNERuntimeIREE::Private::GetTargetPlatformName(TargetPlatform);
	return UE::NNERuntimeIREE::Private::GetModelDataIdentifier(GetRuntimeName(), UNNERuntimeIREERdg::GUID, UNNERuntimeIREERdg::Version, FileId.ToString(EGuidFormats::Digits), PlatformName, "");
}

UNNERuntimeIREERdg::ECanCreateModelRDGStatus UNNERuntimeIREERdg::CanCreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) const
{
	check(ModelData != nullptr);

	using namespace UE::NNERuntimeIREE::RDG::Private;

	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());
	if (!SharedData.IsValid())
	{
		return ECanCreateModelRDGStatus::Fail;
	}

	FModelDataHeader Header{};
	FMemoryReaderView Reader(SharedData->GetView(), /*bIsPersitent =*/ true);
	if (!FModelData::ReadHeaderAndIsSameGuidAndVersion(Reader, GUID, Version, Header))
	{
		return ECanCreateModelRDGStatus::Fail;
	}

	const FString ShaderPlatformName = LexToString(GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel]);
	if (Header.ShaderPlatforms.Find(ShaderPlatformName) == INDEX_NONE)
	{
		return ECanCreateModelRDGStatus::Fail;
	}

	return ECanCreateModelRDGStatus::Ok;
}

TSharedPtr<UE::NNE::IModelRDG> UNNERuntimeIREERdg::CreateModelRDG(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);

	using namespace UE::NNERuntimeIREE;
	using namespace UE::NNERuntimeIREE::Private;
	using namespace UE::NNERuntimeIREE::RDG::Private;

	if (CanCreateModelRDG(ModelData) != ECanCreateModelRDGStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg cannot create a model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelRDG>();
	}

	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());
	check(SharedData.IsValid());

	FModelData IREEModelData{};
	if (!IREEModelData.Load(SharedData->GetView()))
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to load model data %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelRDG>();
	}
	
	const FString ShaderPlatformName = LexToString(GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel]);
	const FIREECompilerRDGBuildTargetResult* BuildTargetResult = IREEModelData.CompilerResult.BuildTargetResults.FindByPredicate([ShaderPlatformName](const FIREECompilerRDGBuildTargetResult &Element) { return Element.ShaderPlatform == ShaderPlatformName; });
	if (!BuildTargetResult)
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to find a matching shader platform for \'%ls\'", *ShaderPlatformName);
		return {};
	}

	TMap<FString, TConstArrayView<uint8>> ExecutableMap;
	for (const FIREECompilerRDGExecutableData& ExecutableData : BuildTargetResult->Executables)
	{
		if (ExecutableData.Name.IsEmpty())
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to load executable map, key is empty");
			return {};
		}

		if (ExecutableData.Data.IsEmpty())
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg failed to load executable map, value for key %ls is empty", *ExecutableData.Name);
			return {};
		}

		ExecutableMap.Emplace(ExecutableData.Name, ExecutableData.Data);
	}

	const FString FileIdString = IREEModelData.Header.FileId.ToString(EGuidFormats::Digits).ToLower();
	const FString RuntimeSubdir = GetRuntimeSubdir();
	const FString SharedLibraryDirFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), GetSharedLibDirPath(UGameplayStatics::GetPlatformName(), GetRuntimeName(), FileIdString), RuntimeSubdir));
	const FString SharedLibrarySubDirFullPath = FPaths::Combine(SharedLibraryDirFullPath, BuildTargetResult->ShaderPlatform);

	TSharedPtr<UE::NNE::IModelRDG> Model = UE::NNERuntimeIREE::RDG::FModel::Make(SharedLibrarySubDirFullPath, BuildTargetResult->VmfbData, IREEModelData.ModuleMetaData, ExecutableMap);
	if (!Model.IsValid())
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREERdg could not initialize the model created from model data with id %ls", *FileIdString);
		return {};
	}

	return Model;
}

bool UNNERuntimeIREERdg::IsAvailable() const
{
#if !WITH_EDITOR
	if(GMaxRHIFeatureLevel != ERHIFeatureLevel::SM6)
	{
		UE_LOGF(LogNNERuntimeIREE, Log, "Minimum feature level required is SM6 for current RHI platform.");
		return false;
	}

	if(!GRHIGlobals.SupportsNative16BitOps)
	{
		UE_LOGF(LogNNERuntimeIREE, Log, "Current RHI platform doesn't support native 16-bit operations.");
		return false;
	}
#endif

	return true;
}

#endif // WITH_NNE_RUNTIME_IREE_RDG

#endif // WITH_NNE_RUNTIME_IREE