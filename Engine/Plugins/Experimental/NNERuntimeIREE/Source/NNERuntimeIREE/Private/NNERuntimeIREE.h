// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "NNERuntime.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeRDG.h"

#include "NNERuntimeIREE.generated.h"

#ifdef WITH_NNE_RUNTIME_IREE
namespace UE::NNERuntimeIREE::Private
{
	class FEnvironment;
}
#endif // WITH_NNE_RUNTIME_IREE

UCLASS()
class UNNERuntimeIREECpu : public UObject, public INNERuntime, public INNERuntimeCPU
{
	GENERATED_BODY()

public:
#ifdef WITH_NNE_RUNTIME_IREE
	static FGuid GUID;
	static int32 Version;

	void Init(TSharedRef<UE::NNERuntimeIREE::Private::FEnvironment> InEnvironment);

	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override;
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual UObject* CreateDefaultRuntimeSettings() const override;
	//~ End INNERuntime Interface

	//~ Begin INNERuntimeCPU Interface
	virtual ECanCreateModelCPUStatus CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelCPU> CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) override;
	//~ End INNERuntimeCPU Interface

private:
	TSharedPtr<UE::NNERuntimeIREE::Private::FEnvironment> Environment;

#else
	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override { return ""; };
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ECanCreateModelDataStatus::Fail; };
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override { return TSharedPtr<UE::NNE::FSharedModelData>(); };
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ""; };
	//~ End INNERuntime Interface

	//~ Begin INNERuntimeCPU Interface
	virtual ECanCreateModelCPUStatus CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const override { return ECanCreateModelCPUStatus::Fail; };
	virtual TSharedPtr<UE::NNE::IModelCPU> CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) override { return TSharedPtr<UE::NNE::IModelCPU>(); };
	//~ End INNERuntimeCPU Interface
#endif // WITH_NNE_RUNTIME_IREE
};

UCLASS()
class UNNERuntimeIREEGpu : public UObject, public INNERuntime, public INNERuntimeGPU
{
	GENERATED_BODY()

public:
#ifdef WITH_NNE_RUNTIME_IREE
	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override;
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	//~ End INNERuntime Interface

	//~ Begin INNERuntimeGPU Interface
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override;
	//~ End INNERuntimeGPU Interface

	virtual bool IsAvailable() const;
	virtual FGuid GetGUID() const;
	virtual int32 GetVersion() const;
#else
	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override { return ""; };
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ECanCreateModelDataStatus::Fail; };
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override { return TSharedPtr<UE::NNE::FSharedModelData>(); };
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ""; };
	//~ End INNERuntime Interface

	//~ Begin INNERuntimeGPU Interface
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override { return ECanCreateModelGPUStatus::Fail; };
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override { return TSharedPtr<UE::NNE::IModelGPU>(); };
	//~ End INNERuntimeGPU Interface
#endif // WITH_NNE_RUNTIME_IREE
};

UCLASS()
class UNNERuntimeIREECuda : public UNNERuntimeIREEGpu
{
	GENERATED_BODY()

public:
#ifdef WITH_NNE_RUNTIME_IREE
	static FGuid GUID;
	static int32 Version;

	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override;
	//~ End INNERuntime Interface

	//~ Begin UNNERuntimeIREEGpu Interface
	virtual bool IsAvailable() const override;
	virtual FGuid GetGUID() const override;
	virtual int32 GetVersion() const override;
	//~ Begin UNNERuntimeIREEGpu Interface
#endif // WITH_NNE_RUNTIME_IREE
};

UCLASS()
class UNNERuntimeIREEVulkan : public UNNERuntimeIREEGpu
{
	GENERATED_BODY()

public:
#ifdef WITH_NNE_RUNTIME_IREE
	static FGuid GUID;
	static int32 Version;

	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override;
	//~ End INNERuntime Interface

	//~ Begin UNNERuntimeIREEGpu Interface
	virtual bool IsAvailable() const override;
	virtual FGuid GetGUID() const override;
	virtual int32 GetVersion() const override;
	//~ Begin UNNERuntimeIREEGpu Interface
#endif // WITH_NNE_RUNTIME_IREE
};

UCLASS()
class UNNERuntimeIREERdg : public UObject, public INNERuntime, public INNERuntimeRDG
{
	GENERATED_BODY()

public:
#if defined(WITH_NNE_RUNTIME_IREE) && defined(WITH_NNE_RUNTIME_IREE_RDG)
	static FGuid GUID;
	static int32 Version;

	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override;
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	//~ End INNERuntime Interface

	//~ Begin INNERuntimeRdg Interface
	virtual ECanCreateModelRDGStatus CanCreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelRDG> CreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) override;
	//~ End INNERuntimeRdg Interface

	bool IsAvailable() const;
#else
	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override { return ""; };
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ECanCreateModelDataStatus::Fail; };
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override { return TSharedPtr<UE::NNE::FSharedModelData>(); };
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ""; };
	//~ End INNERuntime Interface

	//~ Begin INNERuntimeRDG Interface
	virtual ECanCreateModelRDGStatus CanCreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) const override { return ECanCreateModelRDGStatus::Fail; };
	virtual TSharedPtr<UE::NNE::IModelRDG> CreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) override { return TSharedPtr<UE::NNE::IModelRDG>(); };
	//~ End INNERuntimeRDG Interface
#endif // WITH_NNE_RUNTIME_IREE
};