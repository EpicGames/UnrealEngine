// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREECompiler.h"
#include "Logging/LogVerbosity.h"
#include "Misc/CString.h"
#include "Templates/UnrealTemplate.h"

#ifdef WITH_NNE_RUNTIME_IREE
#if WITH_EDITOR

#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "IREEUtils.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNE.h"
#include "NNERuntimeIREELog.h"
#include "NNERuntimeIREESettings.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/SharedPointer.h"

namespace UE::NNERuntimeIREE
{
	namespace CPU
	{
		namespace Private
		{
			FString GetSharedLibraryEntryPointName(const FString& HeaderString)
			{
				FString SearchString = "iree_hal_executable_library_header_t**";
				int32 Start = HeaderString.Find(SearchString);
				if (Start == INDEX_NONE)
				{
					return "";
				}
				Start += SearchString.Len();
				int32 End = HeaderString.Find("(", ESearchCase::CaseSensitive, ESearchDir::FromStart, Start);
				if (End <= Start)
				{
					return "";
				}
				return HeaderString.Mid(Start, End - Start).TrimStartAndEnd();
			}
		} // Private

		FCompiler::FCompiler(const FString& InImporterCommand, const FString& InImporterArguments, const FString& InCompilerCommand, const FString& InLinkerCommand, const FString& InSharedLibExt, TConstArrayView<FBuildTarget> InBuildTargets, const FMD5Hash& InHash) 
			: ImporterCommand(InImporterCommand), ImporterArguments(InImporterArguments), CompilerCommand(InCompilerCommand), LinkerCommand(InLinkerCommand), SharedLibExt(InSharedLibExt), BuildTargets(InBuildTargets), Hash(InHash)
		{

		}

		TUniquePtr<FCompiler> FCompiler::Make(const FString& InTargetPlatformName, const FNNERuntimeIREECpuCompilerSettings& CompilerSettings)
		{
			SCOPED_NAMED_EVENT_TEXT("FCompiler::Make", FColor::Magenta);

			using namespace Private;

			FString PluginDir = FPaths::ConvertRelativePathToFull(*IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetBaseDir());
			FString BuildConfigFileName = FString("IREE_") + UGameplayStatics::GetPlatformName() + "_To_" + InTargetPlatformName + ".json";
			TArray<FString> BuildConfigFilePaths =
			{
				FPaths::Combine(FPaths::ConvertRelativePathToFull(*FPaths::ProjectConfigDir()), BuildConfigFileName),
				FPaths::Combine(PluginDir, "Config", BuildConfigFileName),
				FPaths::Combine(FPaths::ConvertRelativePathToFull(*FPaths::EngineDir()), "Platforms", InTargetPlatformName, "Plugins", UE_PLUGIN_NAME, "Config", BuildConfigFileName),
				FPaths::Combine(FPaths::ConvertRelativePathToFull(*FPaths::EngineDir()), "Platforms", InTargetPlatformName, "Plugins", "Experimental", UE_PLUGIN_NAME, "Config", BuildConfigFileName)
			};

			FString ImporterCommand;
			FString ImporterArguments;
			FString CompilerCommand;
			FString LinkerCommand;
			FString SharedLibExt;
			TArray<FBuildTarget> BuildTargets;
			FMD5Hash Hash;
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			for (FString BuildConfigFilePath : BuildConfigFilePaths)
			{
				if (PlatformFile.FileExists(*BuildConfigFilePath))
				{
					FString BuildConfigFileString;
					if (FFileHelper::LoadFileToString(BuildConfigFileString, *BuildConfigFilePath))
					{
						FBuildConfig BuildConfig;
						if(BuildConfig.FromJson(BuildConfigFileString))
						{
							if (BuildConfig.BuildTargets.IsEmpty())
							{
								UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not find targets in %ls", *BuildConfigFilePath);
								continue;
							}

							FString TmpImporterCommand;
							for (int32 i = 0; i < BuildConfig.ImporterCommand.Num(); i++)
							{
								if (IREEUtils::ResolveEnvironmentVariables(BuildConfig.ImporterCommand[i]))
								{
									BuildConfig.ImporterCommand[i].ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
									BuildConfig.ImporterCommand[i].ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());
									if (PlatformFile.FileExists(*BuildConfig.ImporterCommand[i]))
									{
										TmpImporterCommand = BuildConfig.ImporterCommand[i];
										break;
									}
								}
								else
								{
									UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace environment variables in %ls", *BuildConfig.ImporterCommand[i]);
								}
							}
							if (TmpImporterCommand.IsEmpty())
							{
								UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not find the importer executable in %ls", *BuildConfigFilePath);
								continue;
							}

							FString TmpCompilerCommand;
							for (int32 i = 0; i < BuildConfig.CompilerCommand.Num(); i++)
							{
								if (IREEUtils::ResolveEnvironmentVariables(BuildConfig.CompilerCommand[i]))
								{
									BuildConfig.CompilerCommand[i].ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
									BuildConfig.CompilerCommand[i].ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());
									if (PlatformFile.FileExists(*BuildConfig.CompilerCommand[i]))
									{
										TmpCompilerCommand = BuildConfig.CompilerCommand[i];
										break;
									}
								}
								else
								{
									UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace environment variables in %ls", *BuildConfig.CompilerCommand[i]);
								}
							}
							if (TmpCompilerCommand.IsEmpty())
							{
								UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not find the compiler executable in %ls", *BuildConfigFilePath);
								continue;
							}

							FString TmpLinkerCommand;
							for (int32 i = 0; i < BuildConfig.LinkerCommand.Num(); i++)
							{
								if (IREEUtils::ResolveEnvironmentVariables(BuildConfig.LinkerCommand[i]))
								{
									BuildConfig.LinkerCommand[i].ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
									BuildConfig.LinkerCommand[i].ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());
									if (PlatformFile.FileExists(*BuildConfig.LinkerCommand[i]))
									{
										TmpLinkerCommand = BuildConfig.LinkerCommand[i];
										break;
									}
								}
								else
								{
									UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace environment variables in %ls", *BuildConfig.LinkerCommand[i]);
								}
							}
							if (TmpLinkerCommand.IsEmpty())
							{
								UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not find the linker executable in %ls", *BuildConfigFilePath);
								continue;
							}

							TArray<FBuildTarget> TmpBuildTargets = BuildConfig.BuildTargets;
							FString SettingsToHash;
							for (FBuildTarget& Target : TmpBuildTargets)
							{
								for (const FNNERuntimeIREECpuCompilerFlags& FlagSettings: CompilerSettings.CompilerFlags)
								{
									if (!FlagSettings.Platform.IsEmpty() && FlagSettings.Platform.Compare(InTargetPlatformName, ESearchCase::IgnoreCase) != 0)
									{
										continue;
									}

									if (!FlagSettings.Target.IsEmpty() && FlagSettings.Target.Compare(Target.Name, ESearchCase::IgnoreCase) != 0)
									{
										continue;
									}

									Target.CompilerArguments.AppendChar(' ');
									Target.CompilerArguments.Append(FlagSettings.Flags);

									UE_LOGF(LogNNERuntimeIREE, Verbose, "Updated IREE compiler flags for %ls on %ls: %ls", *Target.Name, *InTargetPlatformName, *Target.CompilerArguments);

									SettingsToHash.Append(FlagSettings.Platform)
										.AppendChar('|')
										.Append(FlagSettings.Target)
										.AppendChar('|')
										.Append(FlagSettings.Flags);

									break;
								}

								SettingsToHash.AppendChar('\n');
							}

							ImporterCommand = TmpImporterCommand;
							ImporterArguments = BuildConfig.ImporterArguments;
							CompilerCommand = TmpCompilerCommand;
							LinkerCommand = TmpLinkerCommand;
							SharedLibExt = BuildConfig.SharedLibExt;
							BuildTargets = MoveTemp(TmpBuildTargets);

							FMD5 HashBuilder{};
							bool bHashSuccess = true;
							UE::IREEUtils::HashAppendString(HashBuilder, BuildConfigFileString);
							UE::IREEUtils::HashAppendString(HashBuilder, SettingsToHash);
							bHashSuccess &= UE::IREEUtils::HashAppendFileStat(HashBuilder, ImporterCommand);
							bHashSuccess &= UE::IREEUtils::HashAppendFileStat(HashBuilder, CompilerCommand);
							bHashSuccess &= UE::IREEUtils::HashAppendFileStat(HashBuilder, LinkerCommand);
							Hash.Set(HashBuilder);
							if (!bHashSuccess)
							{
								UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to create a valid compiler hash, while loading %ls", *BuildConfigFilePath);
							}
							break;
						}
						else
						{
							UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not parse build config file %ls", *BuildConfigFilePath);
						}
					}
					else
					{
						UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not read build config file %ls", *BuildConfigFilePath);
					}
				}
			}
			if (CompilerCommand.IsEmpty() || LinkerCommand.IsEmpty() || BuildTargets.IsEmpty())
			{
				return TUniquePtr<FCompiler>();
			}
			return TUniquePtr<FCompiler>(new FCompiler(ImporterCommand, ImporterArguments, CompilerCommand, LinkerCommand, SharedLibExt, BuildTargets, Hash));
		}

		bool FCompiler::ImportOnnx(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TArray64<uint8>& OutMlirData)
		{
			return IREEUtils::ImportOnnx(ImporterCommand, ImporterArguments, InFileData, InModelName, InOutputDir, OutMlirData);
		}

		bool FCompiler::CompileMlir(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, FNNERuntimeIREECompilerResultCPU& OutCompilerResult)
		{
			SCOPED_NAMED_EVENT_TEXT("FCompiler::CompileMlir", FColor::Magenta);

			using namespace Private;

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			FString InputFilePath = FPaths::Combine(InOutputDir, InModelName) + ".mlir";
			if (!PlatformFile.FileExists(*InputFilePath))
			{
				SCOPED_NAMED_EVENT_TEXT("InputFile", FColor::Magenta);

				FFileHelper::SaveArrayToFile(InFileData, *InputFilePath);
			}

			const FString PluginDir = FPaths::ConvertRelativePathToFull(*IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetBaseDir());

			bool bResult = true;
			OutCompilerResult.Reset();
			TArray<FString> PreviousRelativeDirPaths;
			for (int32 i = 0; i < BuildTargets.Num(); i++)
			{
				FString RelativeDirPath;
				if (BuildTargets.Num() > 1)
				{
					if (!BuildTargets[i].Name.IsEmpty())
					{
						RelativeDirPath = FPaths::MakeValidFileName(BuildTargets[i].Name);
						if (RelativeDirPath.StartsWith(TEXT("Target_")))
						{
							RelativeDirPath.Empty();
						}
					}
					if (RelativeDirPath.IsEmpty() || PreviousRelativeDirPaths.Contains(RelativeDirPath))
					{
						RelativeDirPath = FString::Printf(TEXT("Target_%d"), i);
					}
					PreviousRelativeDirPaths.Add(RelativeDirPath);
				}
				FString IntermediateDirPath = FPaths::Combine(InOutputDir, RelativeDirPath);
				PlatformFile.CreateDirectoryTree(*IntermediateDirPath);
				FString IntermediateFilePathNoExt = FPaths::Combine(IntermediateDirPath, InModelName);
				FString ObjectFilePath = IntermediateFilePathNoExt + ".o";
				FString VmfbFilePath = IntermediateFilePathNoExt + ".vmfb";
				FString SharedLibFilePath = IntermediateFilePathNoExt + SharedLibExt;

				FString CompilerArguments = BuildTargets[i].CompilerArguments;
				if (!IREEUtils::ResolveEnvironmentVariables(CompilerArguments))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace environment variables in %ls", *BuildTargets[i].CompilerArguments);
					bResult = false;
					break;
				}
				if (!IREEUtils::ResolveSdkPaths(CompilerArguments))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace SDK paths in %ls", *BuildTargets[i].CompilerArguments);
					bResult = false;
					break;
				}
				CompilerArguments.ReplaceInline(*FString("${OBJECT_PATH}"), *(FString("\"") + ObjectFilePath + "\""));
				CompilerArguments.ReplaceInline(*FString("${VMFB_PATH}"), *(FString("\"") + VmfbFilePath + "\""));
				CompilerArguments.ReplaceInline(*FString("${INPUT_PATH}"), *(FString("\"") + InputFilePath + "\""));
				CompilerArguments.ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
				CompilerArguments.ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());

				{
					SCOPED_NAMED_EVENT_TEXT("Compile", FColor::Magenta);

					IREEUtils::RunCommand(CompilerCommand, CompilerArguments, PluginDir, IntermediateFilePathNoExt + "_compile-log.txt");
				}

				if (!PlatformFile.FileExists(*ObjectFilePath) || !PlatformFile.FileExists(*VmfbFilePath))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to compile the model \"%ls\" using the command:", *InputFilePath);
					UE_LOGF(LogNNERuntimeIREE, Warning, "\"%ls\" %ls", *CompilerCommand, *CompilerArguments);
					bResult = false;
					break;
				}

				FString LinkerArguments = BuildTargets[i].LinkerArguments;
				if (!IREEUtils::ResolveEnvironmentVariables(LinkerArguments))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace environment variables in %ls", *BuildTargets[i].LinkerArguments);
					bResult = false;
					break;
				}
				if (!IREEUtils::ResolveSdkPaths(LinkerArguments))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not replace SDK paths in %ls", *BuildTargets[i].LinkerArguments);
					bResult = false;
					break;
				}
				LinkerArguments.ReplaceInline(*FString("${OBJECT_PATH}"), *(FString("\"") + ObjectFilePath + "\""));
				LinkerArguments.ReplaceInline(*FString("${SHARED_LIB_PATH}"), *(FString("\"") + SharedLibFilePath + "\""));
				LinkerArguments.ReplaceInline(*FString("${PLUGIN_DIR}"), *PluginDir);
				LinkerArguments.ReplaceInline(*FString("${PROJECT_DIR}"), *FPaths::ProjectDir());

				{
					SCOPED_NAMED_EVENT_TEXT("Link", FColor::Magenta);

					IREEUtils::RunCommand(LinkerCommand, LinkerArguments, PluginDir, IntermediateFilePathNoExt + "_link-log.txt");
				}

				if (!PlatformFile.FileExists(*SharedLibFilePath))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu failed to link the model \"%ls\" using the command:", *InputFilePath);
					UE_LOGF(LogNNERuntimeIREE, Warning, "\"%ls\" %ls", *LinkerCommand, *LinkerArguments);
					bResult = false;
					break;
				}

				FString SharedLibraryEntryPointName = "";
				FString HeaderPath = IntermediateFilePathNoExt + ".h";
				if (!PlatformFile.FileExists(*HeaderPath))
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not find the model header \"%ls\"", *HeaderPath);
					bResult = false;
					break;
				}
				FString HeaderString;
				if (!FFileHelper::LoadFileToString(HeaderString, *HeaderPath) || HeaderString.IsEmpty())
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not read the model header \"%ls\"", *HeaderPath);
					bResult = false;
					break;
				}
				SharedLibraryEntryPointName = GetSharedLibraryEntryPointName(HeaderString);
				if (SharedLibraryEntryPointName.IsEmpty())
				{
					UE_LOGF(LogNNERuntimeIREE, Warning, "UNNERuntimeIREECpu could not find the entry point in model header \"%ls\"", *HeaderPath);
					bResult = false;
					break;
				}

				FNNERuntimeIREEArchitectureInfoCPU ArchitectureInfo;
				ArchitectureInfo.Architecture = BuildTargets[i].Architecture;
				ArchitectureInfo.X86Features = BuildTargets[i].X86Features;
				ArchitectureInfo.RelativeDirPath = RelativeDirPath;
				ArchitectureInfo.SharedLibraryFileName = InModelName + SharedLibExt;
				ArchitectureInfo.VmfbFileName = InModelName + ".vmfb";
				ArchitectureInfo.SharedLibraryEntryPointName = SharedLibraryEntryPointName;
				OutCompilerResult.ArchitectureInfos.Add(MoveTemp(ArchitectureInfo));
			}

			bResult &= !OutCompilerResult.ArchitectureInfos.IsEmpty();

			if (!bResult)
			{
				OutCompilerResult.Reset();
			}

			return bResult;
		}
	} // CPU
} // UE::NNERuntimeIREE

#endif // WITH_EDITOR
#endif // WITH_NNE_RUNTIME_IREE