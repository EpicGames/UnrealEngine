// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Misc/SecureHash.h"

#include "NNERuntimeIREECompiler.generated.h"

struct FNNERuntimeIREECpuCompilerSettings;

USTRUCT()
struct FNNERuntimeIREEArchitectureInfoCPU
{
	GENERATED_BODY()

	UPROPERTY()
	FString Architecture;

	UPROPERTY()
	TArray<FString> X86Features;

	UPROPERTY()
	FString RelativeDirPath;
	
	UPROPERTY()
	FString SharedLibraryFileName;

	UPROPERTY()
	FString VmfbFileName;

	UPROPERTY()
	FString SharedLibraryEntryPointName;
};

USTRUCT()
struct FNNERuntimeIREECompilerResultCPU
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FNNERuntimeIREEArchitectureInfoCPU> ArchitectureInfos;

	void Reset()
	{
		ArchitectureInfos.Reset();
	}
};

#ifdef WITH_NNE_RUNTIME_IREE
#if WITH_EDITOR

namespace UE::NNERuntimeIREE::CPU
{
	struct FBuildTarget : public FJsonSerializable
	{
		FString Name;
		FString Architecture;
		TArray<FString> X86Features;
		FString CompilerArguments;
		FString LinkerArguments;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("Name", Name);
			JSON_SERIALIZE("Architecture", Architecture);
			JSON_SERIALIZE_ARRAY("X86Features", X86Features);
			JSON_SERIALIZE("CompilerArguments", CompilerArguments);
			JSON_SERIALIZE("LinkerArguments", LinkerArguments);
		END_JSON_SERIALIZER
	};

	struct FBuildConfig : public FJsonSerializable
	{
		TArray<FString> ImporterCommand;
		FString ImporterArguments;
		TArray<FString> CompilerCommand;
		TArray<FString> LinkerCommand;
		FString SharedLibExt;
		TArray<FBuildTarget> BuildTargets;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE_ARRAY("ImporterCommand", ImporterCommand);
			JSON_SERIALIZE("ImporterArguments", ImporterArguments);
			JSON_SERIALIZE_ARRAY("CompilerCommand", CompilerCommand);
			JSON_SERIALIZE_ARRAY("LinkerCommand", LinkerCommand);
			JSON_SERIALIZE("SharedLibExt", SharedLibExt);
			JSON_SERIALIZE_ARRAY_SERIALIZABLE("Targets", BuildTargets, FBuildTarget);
		END_JSON_SERIALIZER
	};

	class FCompiler
	{
	private:
		FCompiler(const FString& InImporterCommand, const FString& InImporterArguments, const FString& InCompilerCommand, const FString& InLinkerCommand, const FString& InSharedLibExt, TConstArrayView<FBuildTarget> InBuildTargets, const FMD5Hash& InHash);

	public:
		~FCompiler() {};
		static TUniquePtr<FCompiler> Make(const FString& InTargetPlatformName, const FNNERuntimeIREECpuCompilerSettings& CompilerSettings);

	public:
		const FMD5Hash& GetHash() const { return Hash; }
		bool ImportOnnx(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, TArray64<uint8>& OutMlirData);
		bool CompileMlir(TConstArrayView<uint8> InFileData, const FString& InModelName, const FString& InOutputDir, FNNERuntimeIREECompilerResultCPU& OutCompilerResult);

	private:
		FString ImporterCommand;
		FString ImporterArguments;
		FString CompilerCommand;
		FString LinkerCommand;
		FString SharedLibExt;
		TArray<FBuildTarget> BuildTargets;
		const FMD5Hash Hash;
	};
} // UE::NNERuntimeIREE::CPU

#endif // WITH_EDITOR
#endif // WITH_NNE_RUNTIME_IREE