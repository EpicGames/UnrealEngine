// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEEnvironment.h"

#ifdef WITH_NNE_RUNTIME_IREE

#include "Algo/AllOf.h"
#include "Misc/ScopeLock.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#endif // PLATFORM_MICROSOFT
THIRD_PARTY_INCLUDES_START
#include "iree/task/api.h"
#include "iree/task/executor.h"
#include "iree/task/topology.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // PLATFORM_MICROSOFT

LLM_DEFINE_TAG(NNERuntimeIREE_Global);

namespace UE::NNERuntimeIREE::Private
{

namespace HostAllocatorUtils
{

iree_status_t HostAllocatorImplAlloc(iree_allocator_command_t Command, const iree_allocator_alloc_params_t* Params, void** InoutPtr)
{
	check(Params);
	check(InoutPtr);
	switch (Command)
	{
		case IREE_ALLOCATOR_COMMAND_MALLOC:
			*InoutPtr = FMemory::Malloc(Params->byte_length, IREE_HAL_HEAP_BUFFER_ALIGNMENT);
			break;
		case IREE_ALLOCATOR_COMMAND_CALLOC:
			*InoutPtr = FMemory::MallocZeroed(Params->byte_length, IREE_HAL_HEAP_BUFFER_ALIGNMENT);
			break;
		case IREE_ALLOCATOR_COMMAND_REALLOC:
			*InoutPtr = FMemory::Realloc(*InoutPtr, Params->byte_length, IREE_HAL_HEAP_BUFFER_ALIGNMENT);
			break;
		default:
			check(false);
	}
	if (!*InoutPtr)
	{
		return iree_make_status(IREE_STATUS_RESOURCE_EXHAUSTED, "Host allocator failed during alloc");
	}
	return iree_ok_status();
}

iree_status_t HostAllocatorImpl(void *Self, iree_allocator_command_t Command, const void* Params, void** InoutPtr)
{
	LLM_SCOPE_BYTAG(NNERuntimeIREE_Global);
	check(InoutPtr);
	switch (Command)
	{
		case IREE_ALLOCATOR_COMMAND_MALLOC:
		case IREE_ALLOCATOR_COMMAND_CALLOC:
		case IREE_ALLOCATOR_COMMAND_REALLOC:
			return HostAllocatorImplAlloc(Command, static_cast<const iree_allocator_alloc_params_t*>(Params), InoutPtr);
		case IREE_ALLOCATOR_COMMAND_FREE:
			FMemory::Free(*InoutPtr);
			break;
		default:
			return iree_make_status(IREE_STATUS_UNIMPLEMENTED, "Host allocator command not recognised");
	}
	return iree_ok_status();
}

} // namespace HostAllocatorUtils

iree_allocator_t MakeHostAllocator()
{
	iree_allocator_t Allocator = { NULL, HostAllocatorUtils::HostAllocatorImpl };
	return Allocator;
}

void PrintIREEError(const FString& InMessage, iree_status_t InStatus)
{
	iree_host_size_t TrueLength = 0;
	iree_status_format(InStatus, 0, (char*)nullptr, &TrueLength);
	void* ErrorString = FMemory::Malloc(TrueLength + 1);
	((char*)ErrorString)[TrueLength] = (char)0;
	iree_status_format(InStatus, TrueLength, (char*)ErrorString, &TrueLength);
	UE_LOGF(LogNNERuntimeIREE, Error, "%ls: %ls", *InMessage, *FString(StringCast<TCHAR>(static_cast<const ANSICHAR*>(ErrorString)).Get()));
	FMemory::Free(ErrorString);
}

namespace TaskUtils
{

bool CreateTaskTopology(const TArray<FNNERuntimeIREEThreadingAffinity>& TaskTopologyGroups, TArray<iree_task_topology_t>& TaskTopologies)
{
	if (TaskTopologyGroups.IsEmpty())
	{
		UE_LOGF(LogNNERuntimeIREE, Log, "Using default task topology...");

		const iree_host_size_t MaxCoreCount = 64;
		iree_task_topology_t TaskTopology;
		iree_status_t Status = iree_task_topology_initialize_from_physical_cores(IREE_TASK_TOPOLOGY_NODE_ID_ANY, IREE_TASK_TOPOLOGY_PERFORMANCE_LEVEL_ANY, IREE_TASK_TOPOLOGY_DISTRIBUTION_SCATTER, MaxCoreCount, &TaskTopology);
		if (!iree_status_is_ok(Status))
		{
			Private::PrintIREEError("Failed to initialize task topology", Status);
			return false;
		}

		TaskTopologies.Add(TaskTopology);

		return true;
	}

	const iree_host_size_t NumNodes = iree_task_topology_query_node_count();
	const iree_task_topology_node_id_t CurrentNode = iree_task_topology_query_current_node();

	bool bSuccess = true;

	if (Algo::AllOf(TaskTopologyGroups, [](FNNERuntimeIREEThreadingAffinity Group) { return Group.CoreIndex == -1; }))
	{
		check(!TaskTopologyGroups.IsEmpty());

		const int32 NumCores = TaskTopologyGroups.Num();
		const ENNERuntimeIREEThreadingAffinityGroupSpecifierType Type = TaskTopologyGroups[0].GroupSpecifierType;
		const int32 GroupIndex = TaskTopologyGroups[0].GroupIndex;

		// To keep implementation simple, we are quite restrictive for now (similar to IREE)
		if (!Algo::AllOf(TaskTopologyGroups, [Type](FNNERuntimeIREEThreadingAffinity Group) { return Group.GroupSpecifierType == Type; }))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "All task topology groups require the same specifier type");
			return false;
		}
		if (Type == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Index &&
			!Algo::AllOf(TaskTopologyGroups, [GroupIndex](FNNERuntimeIREEThreadingAffinity Group) { return Group.GroupIndex == GroupIndex; }))
		{
			UE_LOGF(LogNNERuntimeIREE, Warning, "All task topology groups require the same NUMA node index");
			return false;
		}

		const int32 TotalCores = Type == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::All ? NumCores * NumNodes : NumCores;
		const iree_host_size_t MaxCoreCount = TotalCores == 0 ? 64 : TotalCores;

		const uint32 NodeId = Type == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Index ?
			GroupIndex :
				Type == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Current ?
				CurrentNode : IREE_TASK_TOPOLOGY_NODE_ID_ANY;

		iree_task_topology_t TaskTopology;
		iree_status_t Status = iree_task_topology_initialize_from_physical_cores(NodeId, IREE_TASK_TOPOLOGY_PERFORMANCE_LEVEL_ANY, IREE_TASK_TOPOLOGY_DISTRIBUTION_SCATTER, MaxCoreCount, &TaskTopology);
		if (!iree_status_is_ok(Status))
		{
			Private::PrintIREEError("Failed to initialize task topology", Status);
			return false;
		}

		TaskTopologies.Add(TaskTopology);
	}
	else if (Algo::AllOf(TaskTopologyGroups, [](FNNERuntimeIREEThreadingAffinity Group) { return Group.CoreIndex > -1; }))
	{
		TArray<iree_thread_affinity_t> ThreadAffinities;
		for (const FNNERuntimeIREEThreadingAffinity& Group : TaskTopologyGroups)
		{
			TArray<uint32> Nodes;
			// note: if we are allowed to place threads on any NUMA node, we place it on the same as calling thread (same as type 'current')
			if (Group.GroupSpecifierType == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Current || Group.GroupSpecifierType == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Any)
			{
				Nodes.Add(CurrentNode);
			}
			else if (Group.GroupSpecifierType == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::All)
			{
				for (int i = 0; i < NumNodes; ++i)
				{
					Nodes.Add(i);
				}
			}
			else
			{
				check(Group.GroupSpecifierType == ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Index);
				Nodes.Add(Group.GroupIndex);
			}

			for (uint32 n = 0; n < (uint32)Nodes.Num(); ++n)
			{
				check(Group.CoreIndex >= 0);

				ThreadAffinities.Add(iree_thread_affinity_t
					{
						.group_any = 0,
						.group = (uint32_t)Nodes[n],
						.id_assigned = 1,
						.id = static_cast<uint32>(Group.CoreIndex),
						.smt = 1
					});
			}
		}

		iree_task_topology_t TaskTopology;
		iree_status_t Status = iree_task_topology_initialize_from_thread_affinities(ThreadAffinities.Num(), ThreadAffinities.GetData(), &TaskTopology);
		if (!iree_status_is_ok(Status))
		{
			Private::PrintIREEError("Failed to initialize task topology from thread affinities: ", Status);
			bSuccess = false;
		}
		else
		{
			TaskTopologies.Add(TaskTopology);
		}
	}
	else
	{
		UE_LOGF(LogNNERuntimeIREE, Warning, "Inhomogeneous core assignment not supported yet");
	}

	if (!bSuccess)
	{
		for (auto TaskTopology : TaskTopologies)
		{
			iree_task_topology_deinitialize(&TaskTopology);
		}
		TaskTopologies.Empty();
	}

	if (!bSuccess || TaskTopologies.IsEmpty())
	{
		return false;
	}

	return true;
}

#define DUMP_TASK_TOPOLOGY_LOG_CATEGORY Verbose

// Ported 'iree_task_flags_print_action_flag()' from runtime\src\iree\task\api.c to use logging in Unreal Engine
static void DumpTaskTopology(iree_host_size_t TopologyId, const iree_task_topology_t* Topology)
{
	UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "# ===--------------------------------------------------------------===");
	UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "# topology[%llu]: %llu worker groups", static_cast<uint64>(TopologyId), static_cast<uint64>(Topology->group_count));
	UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "# ===--------------------------------------------------------------===\n#");

	for (iree_host_size_t j = 0; j < Topology->group_count; ++j)
	{
		const iree_task_topology_group_t* Group = &Topology->groups[j];

		UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "# group[%d]: '%ls'", Group->group_index, UTF8_TO_TCHAR(Group->name));
		UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#      processor: %u", Group->processor_index);

		if (Group->ideal_thread_affinity.id_assigned)
		{
			UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#       affinity: group=%u, id=%u, smt=%u", Group->ideal_thread_affinity.group, Group->ideal_thread_affinity.id, Group->ideal_thread_affinity.smt);
		}
		else
		{
			UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#       affinity: (unassigned)");
		}

		UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#  caches: l1d=%u, l2d=%u", Group->caches.l1_data, Group->caches.l2_data);

		if (Group->constructive_sharing_mask == 0)
		{
			UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#  last level cache sharing: (none)");
		}
		else if (Group->constructive_sharing_mask == IREE_TASK_TOPOLOGY_GROUP_MASK_ALL)
		{
			UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#  last level cache sharing: (all/undefined)");
		}
		else
		{
			const uint32 ShareCount = iree_math_count_ones_u64(Group->constructive_sharing_mask);

			FString MaskString;
			for (iree_host_size_t ic = 0, jc = 0; ic < IREE_TASK_TOPOLOGY_GROUP_BIT_COUNT; ++ic)
			{
				if ((Group->constructive_sharing_mask >> ic) & 1)
				{
					if (jc++ > 0)
					{
						MaskString.Append(TEXT(", "));
					}

					MaskString.Append(FString::Printf(TEXT("%llu"), static_cast<uint64>(ic)));
				}
			}

			UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#  last level cache sharing: %u group(s): %ls", ShareCount, *MaskString);
		}

		UE_LOGF(LogNNERuntimeIREE, DUMP_TASK_TOPOLOGY_LOG_CATEGORY, "#");
	}
}

} // namespace TaskUtils

// Unique wrapper around iree_task_executor*
class FTaskExecutor
{
public:
	explicit FTaskExecutor(iree_task_executor_t* TaskExecutor) : TaskExecutor(TaskExecutor)
	{
		check(TaskExecutor != nullptr);
		iree_task_executor_retain(TaskExecutor);
	}

	~FTaskExecutor()
	{
		if (TaskExecutor)
		{
			iree_task_executor_release(TaskExecutor);
		}
	}

	// Disallow copying
	FTaskExecutor(const FTaskExecutor&) = delete;
	FTaskExecutor& operator=(const FTaskExecutor&) = delete;

	// Moving
	FTaskExecutor(FTaskExecutor&& Other) noexcept : TaskExecutor(Other.TaskExecutor)
	{
		Other.TaskExecutor = nullptr;
	}

	FTaskExecutor& operator=(FTaskExecutor&& Other) noexcept
	{
		if (this != &Other)
		{
			if (TaskExecutor)
			{
				iree_task_executor_release(TaskExecutor);
			}

			TaskExecutor = Other.TaskExecutor;
			Other.TaskExecutor = nullptr;
		}
		return *this;
	}

	iree_task_executor_t* Get() const
	{
		return TaskExecutor;
	}

private:
	iree_task_executor_t* TaskExecutor;
};

struct FEnvironment::FInternalState
{
	TArray<FTaskExecutor> TaskExecutors;
};

FEnvironment::FEnvironment() = default;
FEnvironment::~FEnvironment() = default;

void FEnvironment::Configure(const FConfig& InConfig)
{
	FScopeLock ScopeLock(&CriticalSection);

#if WITH_EDITOR
	// For reloading need to make sure thread pool is release before we create a new one
	InternalState.Reset();
#endif // WITH_EDITOR

	checkf(!InternalState.IsValid(), TEXT("IREE environment already created!"));

	Config = InConfig;
}

TArray<iree_task_executor_t*> FEnvironment::GetTaskExecutors() const
{
	FScopeLock ScopeLock(&CriticalSection);

	if (!InternalState.IsValid())
	{
		if (!Create())
		{
			return {};
		}
	}
	
	check(InternalState.IsValid());
	check(!InternalState->TaskExecutors.IsEmpty());

	TArray<iree_task_executor_t*> Result;
	Result.SetNumUninitialized(InternalState->TaskExecutors.Num());

	for (int32 i = 0; i < InternalState->TaskExecutors.Num(); i++)
	{
		Result[i] = InternalState->TaskExecutors[i].Get();
	}

	return Result;
}

bool FEnvironment::Create() const
{
	checkf(!Config.ThreadingOptions.bIsSingleThreaded, TEXT("Can not request task executors in single threaded mode!"));
	checkf(!InternalState.IsValid(), TEXT("IREE environment already created!"));
	
	TArray<iree_task_topology_t> TaskTopologies;
	if (!TaskUtils::CreateTaskTopology(Config.ThreadingOptions.TaskTopology.TaskTopologyGroups, TaskTopologies))
	{
		return false;
	}
	
	check(!TaskTopologies.IsEmpty());

	iree_allocator_t HostAllocator = MakeHostAllocator();

	iree_task_executor_options_t TaskExecutorOptions;
	iree_task_executor_options_initialize(&TaskExecutorOptions);

	iree_status_t Status = iree_ok_status();

	TArray<FTaskExecutor> TaskExecutors;
	TaskExecutors.SetNumZeroed(TaskTopologies.Num());
	for (int i = 0; i < TaskTopologies.Num(); ++i)
	{
		TaskUtils::DumpTaskTopology(i, &TaskTopologies[i]);

		iree_task_executor_t* TaskExecutor = nullptr;
		Status = iree_task_executor_create(TaskExecutorOptions, &TaskTopologies[i], HostAllocator, &TaskExecutor);
		if (!iree_status_is_ok(Status)) {
			Private::PrintIREEError("Failed to initialize task executor", Status);
			break;
		}

		TaskExecutors[i] = FTaskExecutor(TaskExecutor);

		// task executor is managed by wrapper class
		iree_task_executor_release(TaskExecutor);
	}

	// Now we don't need the topologies anymore
	for (iree_task_topology_t TaskTopology : TaskTopologies)
	{
		iree_task_topology_deinitialize(&TaskTopology);
	}

	if (!iree_status_is_ok(Status))
	{
		// Note:
		// - we already logged the error status
		// - already successfully created task executors are in local variable 'TaskExecutors' which on return will be destroyed and releases them properly
		return false;
	}

	InternalState = MakeUnique<FInternalState>();
	InternalState->TaskExecutors = MoveTemp(TaskExecutors);

	// 
	// WITH_EDITOR After this point, if we created a new thread pool, all existing ModelInstances should be created again to use the new one
	// note: they still can use the old ones because iree devices hold a reference to the iree task exectutors
	// 

	return true;
}

} // UE::NNERuntimeIREE::Private

#endif // WITH_NNE_RUNTIME_IREE