// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE

#include "HAL/CriticalSection.h"
#include "NNERuntimeIREELog.h"
#include "NNERuntimeIREESettings.h"

struct iree_task_executor_t;
struct iree_allocator_t;
typedef struct iree_status_handle_t* iree_status_t;

namespace UE::NNERuntimeIREE::Private
{

// Make IREE host allocator that uses Unreal Engine FMemory
iree_allocator_t MakeHostAllocator();

// Print IREE error formatted
void PrintIREEError(const FString& InMessage, iree_status_t InStatus);

// IREE Environment that holds global state, e.g. containing global thread pool etc.
class FEnvironment
{
public:
	struct FConfig
	{
		FNNERuntimeIREEThreadingOptions ThreadingOptions;
	};

	FEnvironment();
	~FEnvironment();

	void Configure(const FConfig& InConfig);

	TArray<iree_task_executor_t*> GetTaskExecutors() const;

	FConfig GetConfig() const { return Config; }

private:
	bool Create() const;

	FConfig Config{};

	struct FInternalState;
	mutable TUniquePtr<FInternalState> InternalState;

	mutable FCriticalSection CriticalSection;
};

} // UE::NNERuntimeIREE::Private

#endif // WITH_NNE_RUNTIME_IREE