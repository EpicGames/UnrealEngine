// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEMetaData.h"

#include "Containers/Array.h"
#include "Internationalization/Regex.h"
#include "NNE.h"
#include "NNERuntimeIREELog.h"
#include "Serialization/CustomVersion.h"

// We register our own error handler for the case when exceptions are diabled
#ifdef NNEMLIR_NO_EXCEPTIONS
#define NNEMLIR_CXX_API_THROW(Message) \
	do { \
		UE_LOGF(LogNNERuntimeIREE, Fatal, "%s", Message.c_str()); \
		abort(); \
	} while (false)
#endif

#include "NNEMlirTools_cxx_api.h"

namespace UE::NNERuntimeIREE
{
	namespace ModuleMetaData::Private
	{
		enum Version : uint32
		{
			V0 = 0, // Initial
			// New versions can be added above this line
			VersionPlusOne,
			Latest = VersionPlusOne - 1
		};
		const FGuid GUID(0x2f9ffd31, 0x12b817cd, 0x627855bf, 0x5e405720);
		FCustomVersionRegistration Version(GUID, Version::Latest, TEXT("NNERuntimeIREEModuleMetaDataVersion"));// Always save with the latest version

		ENNETensorDataType ConvertTypeString(const FString& TypeString)
		{
			if (TypeString.StartsWith("char"))
			{
				return ENNETensorDataType::Char;
			}
			if (TypeString.StartsWith("bool") || TypeString.StartsWith("i1"))
			{
				return ENNETensorDataType::Boolean;
			}
			else if (TypeString.StartsWith("half"))
			{
				return ENNETensorDataType::Half;
			}
			else if (TypeString.StartsWith("bf16"))
			{
				return ENNETensorDataType::BFloat16;
			}
			else if (TypeString.StartsWith("f"))
			{
				if (TypeString.StartsWith("f16"))
				{
					return ENNETensorDataType::Half;
				}
				else if (TypeString.StartsWith("float") || TypeString.StartsWith("f32"))
				{
					return ENNETensorDataType::Float;
				}
				else if (TypeString.StartsWith("f64"))
				{
					return ENNETensorDataType::Double;
				}
			}
			else if (TypeString.StartsWith("double"))
			{
				return ENNETensorDataType::Double;
			}
			else if (TypeString.StartsWith("i") || TypeString.StartsWith("si"))
			{
				if (TypeString.EndsWith("i8"))
				{
					return ENNETensorDataType::Int8;
				}
				else if (TypeString.EndsWith("i16"))
				{
					return ENNETensorDataType::Int16;
				}
				else if (TypeString.EndsWith("i32") || TypeString.EndsWith("int"))
				{
					return ENNETensorDataType::Int32;
				}
				else if (TypeString.EndsWith("i64"))
				{
					return ENNETensorDataType::Int64;
				}
			}
			else if (TypeString.StartsWith("ui"))
			{
				if (TypeString.EndsWith("i8"))
				{
					return ENNETensorDataType::UInt8;
				}
				else if (TypeString.EndsWith("i16"))
				{
					return ENNETensorDataType::UInt16;
				}
				else if (TypeString.EndsWith("i32"))
				{
					return ENNETensorDataType::UInt32;
				}
				else if (TypeString.EndsWith("i64"))
				{
					return ENNETensorDataType::UInt64;
				}
			}
			return ENNETensorDataType::None;
		}
	} // ModuleMetaData::Private

	void FModuleMetaData::Serialize(FArchive& Ar)
	{
		// Store the asset version (no effect in load)
		Ar.UsingCustomVersion(UE::NNERuntimeIREE::ModuleMetaData::Private::GUID);

		if (Ar.IsSaving() || Ar.IsCountingMemory()) 
		{
			int32 NumItems = FunctionMetaData.Num();
			Ar << NumItems;
			for (int32 i = 0; i < NumItems; i++)
			{
				Ar << FunctionMetaData[i].Name;

				int32 NumInputs = FunctionMetaData[i].InputDescs.Num();
				Ar << NumInputs;
				for (int32 j = 0; j < NumInputs; j++)
				{
					FString Name = FunctionMetaData[i].InputDescs[j].GetName();
					Ar << Name;

					ENNETensorDataType Type = FunctionMetaData[i].InputDescs[j].GetDataType();
					Ar << Type;

					TArray<int32> Shape = (TArray<int32>)FunctionMetaData[i].InputDescs[j].GetShape().GetData();
					Ar << Shape;
				}

				int32 NumOutputs = FunctionMetaData[i].OutputDescs.Num();
				Ar << NumOutputs;
				for (int32 j = 0; j < NumOutputs; j++)
				{
					FString Name = FunctionMetaData[i].OutputDescs[j].GetName();
					Ar << Name;

					ENNETensorDataType Type = FunctionMetaData[i].OutputDescs[j].GetDataType();
					Ar << Type;

					TArray<int32> Shape = (TArray<int32>)FunctionMetaData[i].OutputDescs[j].GetShape().GetData();
					Ar << Shape;
				}
			}
		}
		else
		{
			int32 NumItems = 0;
			int32 NumInputs = 0;
			int32 NumOutputs = 0;
			UE::NNERuntimeIREE::FFunctionMetaData MetaData;
			FString Name;
			ENNETensorDataType Type;
			TArray<int32> Shape;

			switch (Ar.CustomVer(UE::NNERuntimeIREE::ModuleMetaData::Private::GUID))
			{
			case UE::NNERuntimeIREE::ModuleMetaData::Private::Version::V0:
				Ar << NumItems;
				FunctionMetaData.SetNum(NumItems, EAllowShrinking::Yes);
				for (int32 i = 0; i < NumItems; i++)
				{
					Ar << MetaData.Name;

					Ar << NumInputs;
					MetaData.InputDescs.Empty();
					for (int32 j = 0; j < NumInputs; j++)
					{
						Ar << Name;
						Ar << Type;
						Ar << Shape;
						MetaData.InputDescs.Add(UE::NNE::FTensorDesc::Make(Name, UE::NNE::FSymbolicTensorShape::Make(Shape), Type));
					}

					Ar << NumOutputs;
					MetaData.OutputDescs.Empty();
					for (int32 j = 0; j < NumOutputs; j++)
					{
						Ar << Name;
						Ar << Type;
						Ar << Shape;
						MetaData.OutputDescs.Add(UE::NNE::FTensorDesc::Make(Name, UE::NNE::FSymbolicTensorShape::Make(Shape), Type));
					}

					FunctionMetaData[i] = MetaData;
				}
				break;
			default:
				UE_LOGF(LogNNERuntimeIREE, Error, "UNNERuntimeIREEModuleMetaData: Unknown asset version %d: Deserialisation failed, please reimport the original model.", Ar.CustomVer(UE::NNERuntimeIREE::ModuleMetaData::Private::GUID));
				break;
			}
		}
	}

	bool FModuleMetaData::ParseFromBuffer(TConstArrayView64<uint8> Buffer)
	{
		using namespace UE::NNE;
		using namespace UE::NNERuntimeIREE::ModuleMetaData::Private;

		const char* BufferPtr = reinterpret_cast<const char*>(Buffer.GetData());
		const size_t BufferSize = static_cast<size_t>(Buffer.Num());

#if WITH_EDITOR
		try
#endif // WITH_EDITOR
		{
			NNEMlirTools::Context Ctx;
			const NNEMlirTools::Module Mod = NNEMlirTools::Module::ParseFromBuffer(Ctx, BufferPtr, BufferSize);

			auto MakeValueName = [] (const FString& Name, const FString& DefaultPrefix, int32 DefaultIndex)
			{
				return !Name.IsEmpty() ? Name : (DefaultPrefix + FString::FromInt(DefaultIndex));
			};

			auto GetShape = [] (const NNEMlirTools::Value& Val)
			{
				const int32 Rank = static_cast<int32>(Val.GetRank());

				TArray<int64_t> Shape64;
				Shape64.SetNumUninitialized(Rank);
				Val.GetShape(Shape64.GetData(), Rank);

				TArray<int32> Result;
				Result.SetNumUninitialized(Rank);

				for (int i = 0; i < Rank; i++)
				{
					Result[i] = static_cast<int32>(Shape64[i]);
				}

				return Result;
			};

			auto GetTensorDesc = [&GetShape] (const FString& Name, const NNEMlirTools::Value& Val)
			{
				const TArray<int32> Shape = GetShape(Val);
				FSymbolicTensorShape SymbolicShape = FSymbolicTensorShape::Make(Shape);

				const FString TypeStr = Val.GetElementType().c_str();
				check(!TypeStr.IsEmpty());

				const ENNETensorDataType DataType = ConvertTypeString(TypeStr);

				return FTensorDesc::Make(Name, SymbolicShape, DataType);
			};

			const int32 NumFun = Mod.GetFunctionCount();
			FunctionMetaData.Reset(NumFun);

			for (int32 i = 0; i < NumFun; i++)
			{
				const NNEMlirTools::Function Fun = Mod.GetFunction(i);

				UE::NNERuntimeIREE::FFunctionMetaData MetaData;
				MetaData.Name = Fun.GetName().c_str();
				check(!MetaData.Name.IsEmpty());

				const int32 NumInputs = Fun.GetInputCount();
				MetaData.InputDescs.Reserve(NumInputs);

				for (int j = 0; j < NumInputs; j++)
				{
					const NNEMlirTools::Value Input = Fun.GetInput(j);
					const FString Name = MakeValueName(Input.GetName().c_str(), TEXT("arg"), j);

					MetaData.InputDescs.Add(GetTensorDesc(Name, Input));
				}

				const int32 NumOutputs = Fun.GetResultCount();
				MetaData.OutputDescs.Reserve(NumOutputs);
			
				for (int j = 0; j < NumOutputs; j++)
				{
					const NNEMlirTools::Value Output = Fun.GetResult(j);
					const FString Name = MakeValueName(Output.GetName().c_str(), TEXT("res"), j);

					MetaData.OutputDescs.Add(GetTensorDesc(Name, Output));
				}

				FunctionMetaData.Add(MoveTemp(MetaData));
			}
		}
#if WITH_EDITOR
		catch (const std::exception& Exception)
		{
			UE_LOGF(LogNNERuntimeIREE, Error, "Parse MLIR exception: %s", Exception.what());
			return false;
		}
		catch (...)
		{
			UE_LOGF(LogNNERuntimeIREE, Error, "Parse MLIR unknown exception.");
			return false;
		}
#endif // WITH_EDITOR

		return true;
	}
} // UE::NNERuntimeIREE
