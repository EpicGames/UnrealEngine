// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEModel.h"

#ifdef WITH_NNE_RUNTIME_IREE

#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "IREEDriverRDG.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeIREEEnvironment.h"
#include "NNERuntimeIREELog.h"
#include "NNERuntimeIREESettings.h"
#include "NNERuntimeIREETensor.h"
#include "NNEStatus.h"
#include "RenderGraphUtils.h"
#include "Serialization/Archive.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#endif // PLATFORM_MICROSOFT
THIRD_PARTY_INCLUDES_START
#include "iree/hal/drivers/local_sync/sync_device.h"
#include "iree/hal/drivers/local_task/task_device.h"
#include "iree/hal/local/loaders/static_library_loader.h"
#include "iree/modules/hal/types.h"
#include "iree/runtime/call.h"
#include "iree/runtime/instance.h"
#include "iree/runtime/session.h"
#include "iree/vm/bytecode/module.h"
#include "iree/vm/list.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // PLATFORM_MICROSOFT

LLM_DEFINE_TAG(NNERuntimeIREE_Cpu);

BEGIN_SHADER_PARAMETER_STRUCT(FIREESessionRDGParameters, )
	RDG_BUFFER_ACCESS_ARRAY(InputBuffers)
	RDG_BUFFER_ACCESS_ARRAY(OutputBuffers)
END_SHADER_PARAMETER_STRUCT()

DECLARE_GPU_STAT_NAMED(FNNERuntimeIREERDGEnqueue, TEXT("NNERuntimeIREERdg.EnqueueRDG"));
DECLARE_GPU_STAT_NAMED(FNNERuntimeIREERDGCall, TEXT("NNERuntimeIREERdg.EnqueueRDG.Call"));

namespace UE::NNERuntimeIREE
{
	namespace Private
	{
		iree_hal_element_types_t NNEToIREEType(ENNETensorDataType Type)
		{
			switch (Type)
			{
			case ENNETensorDataType::None:
				return IREE_HAL_ELEMENT_TYPE_NONE;
			case ENNETensorDataType::Char:
				return IREE_HAL_ELEMENT_TYPE_UINT_8;
			case ENNETensorDataType::Boolean:
				return IREE_HAL_ELEMENT_TYPE_BOOL_8;
			case ENNETensorDataType::Half:
				return IREE_HAL_ELEMENT_TYPE_FLOAT_16;
			case ENNETensorDataType::Float:
				return IREE_HAL_ELEMENT_TYPE_FLOAT_32;
			case ENNETensorDataType::Double:
				return IREE_HAL_ELEMENT_TYPE_FLOAT_64;
			case ENNETensorDataType::Int8:
				return IREE_HAL_ELEMENT_TYPE_INT_8;
			case ENNETensorDataType::Int16:
				return IREE_HAL_ELEMENT_TYPE_INT_16;
			case ENNETensorDataType::Int32:
				return IREE_HAL_ELEMENT_TYPE_INT_32;
			case ENNETensorDataType::Int64:
				return IREE_HAL_ELEMENT_TYPE_INT_64;
			case ENNETensorDataType::UInt8:
				return IREE_HAL_ELEMENT_TYPE_UINT_8;
			case ENNETensorDataType::UInt16:
				return IREE_HAL_ELEMENT_TYPE_UINT_16;
			case ENNETensorDataType::UInt32:
				return IREE_HAL_ELEMENT_TYPE_UINT_32;
			case ENNETensorDataType::UInt64:
				return IREE_HAL_ELEMENT_TYPE_UINT_64;
			case ENNETensorDataType::Complex64:
				return IREE_HAL_ELEMENT_TYPE_COMPLEX_FLOAT_64;
			case ENNETensorDataType::Complex128:
				return IREE_HAL_ELEMENT_TYPE_COMPLEX_FLOAT_128;
			case ENNETensorDataType::BFloat16:
				return IREE_HAL_ELEMENT_TYPE_BFLOAT_16;
			default:
				return IREE_HAL_ELEMENT_TYPE_NONE;
			}
		}

		static bool ShapeEqualDims(const UE::NNE::FTensorShape& Shape, const iree_hal_dim_t* Dims, iree_host_size_t Rank)
		{
			if (Shape.Rank() != Rank)
			{
				return false;
			}
			TConstArrayView<uint32> ShapeData = Shape.GetData();
			for (int i = 0; i < Rank; i++)
			{
				if (ShapeData[i] != Dims[i])
				{
					return false;
				}
			}
			return true;
		}

		static UE::NNE::FTensorShape DimsToShape(const iree_hal_dim_t* Dims, iree_host_size_t Rank)
		{
			uint32 Shape[UE::NNE::FTensorShape::MaxRank];
			int32 ShapeRank = FMath::Min((int32)Rank, UE::NNE::FTensorShape::MaxRank);
			for (int32 ShapeIndex = 0; ShapeIndex < ShapeRank; ShapeIndex++)
			{
				Shape[ShapeIndex] = (uint32)Dims[ShapeIndex];
			}
			return UE::NNE::FTensorShape::Make(TConstArrayView<uint32>(Shape, ShapeRank));
		}

		class FInstance
		{
		private:
			iree_runtime_instance_t* Instance;
			static TWeakPtr<FInstance> WeakInstancePtr;
			static FCriticalSection CriticalSection;

			FInstance(iree_runtime_instance_t* InInstance) : Instance(InInstance)
			{
				check(InInstance);
			}

		public:
			~FInstance()
			{
				iree_runtime_instance_release(Instance);
			}

			static TSharedPtr<FInstance> GetInstance()
			{
				FScopeLock ScopeLock(&CriticalSection);

				if (WeakInstancePtr.IsValid())
				{
					return WeakInstancePtr.Pin();
				}

				iree_status_t Status = iree_ok_status();
				if (!iree_status_is_ok(Status))
				{
					iree_status_free(Status);
					return TSharedPtr<FInstance>();
				}

				iree_runtime_instance_options_t InstanceOptions;
				iree_runtime_instance_options_initialize(&InstanceOptions);
				iree_runtime_instance_options_use_all_available_drivers(&InstanceOptions);

				iree_runtime_instance_t* Instance = nullptr;
				Status = iree_runtime_instance_create(&InstanceOptions, MakeHostAllocator(), &Instance);
				if (!iree_status_is_ok(Status))
				{
					Private::PrintIREEError("CPU instance: Failed to create the instance", Status);

					if (Instance)
					{
						iree_runtime_instance_release(Instance);
					}

					iree_status_free(Status);
					return TSharedPtr<FInstance>();
				}

				TSharedPtr<FInstance> SharedInstance = TSharedPtr<FInstance>(new FInstance(Instance));
				WeakInstancePtr = SharedInstance;

				iree_status_free(Status);
				return SharedInstance;
			}

			bool CreateModule(TConstArrayView<uint8> InVmfbDataView, iree_vm_module_t** OutModule)
			{
				FScopeLock ScopeLock(&CriticalSection);

				check(!InVmfbDataView.IsEmpty());
				check(OutModule);

				iree_status_t Status = iree_ok_status();
				check(iree_status_is_ok(Status));

				const iree_const_byte_span_t ModuleData = iree_make_const_byte_span(InVmfbDataView.GetData(), InVmfbDataView.Num());

				iree_vm_module_t* Module = nullptr;
				Status = iree_vm_bytecode_module_create(iree_runtime_instance_vm_instance(Instance), IREE_VM_BYTECODE_MODULE_FLAG_NONE, ModuleData, iree_allocator_null(), GetHostAllocator(), &Module);
				if (!iree_status_is_ok(Status))
				{
					Private::PrintIREEError("CPU instance: Failed to create the module", Status);

					if (Module)
					{
						iree_vm_module_release(Module);
					}

					iree_status_free(Status);
					return false;
				}

				*OutModule = Module;

				iree_status_free(Status);
				return true;
			}

			bool CreateSyncDevice(void* InLibraryQueryFuntionPointer, iree_hal_device_t** OutDevice)
			{
				FScopeLock ScopeLock(&CriticalSection);

				check(InLibraryQueryFuntionPointer);
				check(OutDevice);

				iree_status_t Status = iree_ok_status();
				check(iree_status_is_ok(Status));

				iree_allocator_t HostAllocator = GetHostAllocator();

				iree_hal_executable_loader_t* LibraryLoader = nullptr;
				const iree_hal_executable_library_query_fn_t LibraryList[] = { (iree_hal_executable_library_query_fn_t)InLibraryQueryFuntionPointer };
				Status = iree_hal_static_library_loader_create(IREE_ARRAYSIZE(LibraryList), LibraryList, iree_hal_executable_import_provider_null(), HostAllocator, &LibraryLoader);
				if (!iree_status_is_ok(Status))
				{
					Private::PrintIREEError("CPU instance: Failed to create the library loader", Status);

					if (LibraryLoader)
					{
						iree_hal_executable_loader_release(LibraryLoader);
					}

					iree_status_free(Status);
					return false;
				}

				iree_hal_allocator_t* DeviceAllocator = nullptr;
				iree_string_view_t Identifier = iree_make_cstring_view("local-sync");
				Status = iree_hal_allocator_create_heap(Identifier, HostAllocator, HostAllocator, &DeviceAllocator);
				if (!iree_status_is_ok(Status))
				{
					Private::PrintIREEError("CPU instance: Failed to create the device allocator", Status);

					if (DeviceAllocator)
					{
						iree_hal_allocator_release(DeviceAllocator);
					}

					iree_hal_executable_loader_release(LibraryLoader);
					iree_status_free(Status);
					return false;
				}

				iree_hal_device_t* Device = nullptr;
				iree_hal_sync_device_params_t DeviceParams;
				iree_hal_sync_device_params_initialize(&DeviceParams);
				Status = iree_hal_sync_device_create(Identifier, &DeviceParams, 1, &LibraryLoader, DeviceAllocator, HostAllocator, &Device);
				if (!iree_status_is_ok(Status))
				{
					Private::PrintIREEError("CPU instance: Failed to create the device", Status);

					if (Device)
					{
						iree_hal_device_release(Device);
					}

					iree_hal_allocator_release(DeviceAllocator);
					iree_hal_executable_loader_release(LibraryLoader);
					iree_status_free(Status);
					return false;
				}

				iree_hal_allocator_release(DeviceAllocator);
				iree_hal_executable_loader_release(LibraryLoader);

				*OutDevice = Device;

				iree_status_free(Status);
				return true;
			}

			bool CreateTaskDevice(UE::NNERuntimeIREE::Private::FEnvironment& InEnvironment, void* InLibraryQueryFuntionPointer, iree_hal_device_t** OutDevice)
			{
				TArray<iree_task_executor_t*> TaskExecutors = InEnvironment.GetTaskExecutors();
				if (TaskExecutors.IsEmpty())
				{
					return false;
				}

				iree_status_t Status = iree_ok_status();

				iree_hal_task_device_params_t TaskDeviceParams;
				iree_hal_task_device_params_initialize(&TaskDeviceParams);

				const iree_hal_executable_library_query_fn_t Libraries[] = { (iree_hal_executable_library_query_fn_t)InLibraryQueryFuntionPointer };
				iree_hal_executable_loader_t* LibraryLoader = NULL;
				if (iree_status_is_ok(Status)) {
					Status = iree_hal_static_library_loader_create(IREE_ARRAYSIZE(Libraries), Libraries, iree_hal_executable_import_provider_null(), GetHostAllocator(), &LibraryLoader);
					if (!iree_status_is_ok(Status))
					{
						Private::PrintIREEError("CPU Task instance: Failed to create the static library loader", Status);
						LibraryLoader = NULL;
					}
				}

				iree_string_view_t Identifier = iree_make_cstring_view("local-task");
				iree_hal_allocator_t* DeviceAllocator = NULL;
				if (iree_status_is_ok(Status)) {
					Status = iree_hal_allocator_create_heap(Identifier, GetHostAllocator(), GetHostAllocator(), &DeviceAllocator);
					if (!iree_status_is_ok(Status))
					{
						Private::PrintIREEError("CPU Task instance: Failed to create the heap allocator", Status);
						DeviceAllocator = NULL;
					}
				}
				
				if (iree_status_is_ok(Status)) {
					Status = iree_hal_task_device_create(
						Identifier,
						&TaskDeviceParams,
						TaskExecutors.Num(),
						TaskExecutors.GetData(),
						/*loader_count=*/1,
						&LibraryLoader,
						DeviceAllocator,
						GetHostAllocator(),
						OutDevice);
					if (!iree_status_is_ok(Status))
					{
						Private::PrintIREEError("CPU Task instance: Failed to create the task device", Status);
					}
				}

				if (DeviceAllocator) iree_hal_allocator_release(DeviceAllocator);
				if (LibraryLoader) iree_hal_executable_loader_release(LibraryLoader);

				bool bResult = iree_status_is_ok(Status);
				iree_status_free(Status);

				return bResult;
			}

			bool CreateSession(iree_hal_device_t* InDevice, iree_runtime_session_t** OutSession)
			{
				FScopeLock ScopeLock(&CriticalSection);

				check(InDevice);
				check(OutSession);

				iree_status_t Status = iree_ok_status();
				check(iree_status_is_ok(Status));

				iree_runtime_session_options_t SessionOptions;
				iree_runtime_session_options_initialize(&SessionOptions);

				iree_runtime_session_t* Session = nullptr;
				Status = iree_runtime_session_create_with_device(Instance, &SessionOptions, InDevice, GetHostAllocator(), &Session);
				if (!iree_status_is_ok(Status))
				{
					Private::PrintIREEError("Failed to create the session", Status);

					if (Session)
					{
						iree_runtime_session_release(Session);
					}

					iree_status_free(Status);
					return false;
				}

				*OutSession = Session;

				iree_status_free(Status);
				return true;
			}

			iree_allocator_t GetHostAllocator()
			{
				FScopeLock ScopeLock(&CriticalSection);
				return iree_runtime_instance_host_allocator(Instance);
			}
		};
		TWeakPtr<FInstance> FInstance::WeakInstancePtr;
		FCriticalSection FInstance::CriticalSection;

		class FModule
		{
		private:
			TSharedRef<FInstance> Instance;
			TSharedRef<UE::NNE::FSharedModelData> ModelData;
			iree_vm_module_t* Module;
			TArray<UE::NNERuntimeIREE::FFunctionMetaData> FunctionMetaData;
			static TMap<FString, TWeakPtr<FModule>> Modules;

			FModule(TSharedRef<FInstance> InInstance, TSharedRef<UE::NNE::FSharedModelData> InModelData, iree_vm_module_t* InModule, TConstArrayView<UE::NNERuntimeIREE::FFunctionMetaData> InFunctionMetaData)
				: Instance(InInstance), ModelData(InModelData), Module(InModule), FunctionMetaData(InFunctionMetaData)
			{
				check(!ModelData->GetView().IsEmpty());
				check(InModule);
				check(!InFunctionMetaData.IsEmpty());
			}

		public:
			~FModule()
			{
				iree_vm_module_release(Module);
			}

			static TSharedPtr<UE::NNE::FSharedModelData> GetVmfbAsModelData(const FString& InVmfbPath)
			{
				check(!InVmfbPath.IsEmpty());

				TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*InVmfbPath));
				if (!Reader)
				{
					UE_LOGF(LogNNERuntimeIREE, Error, "CPU module: Failed to open the vmfb data file '%ls'", *InVmfbPath);
					return {};
				}
				int64 DataSize = Reader->TotalSize();
				if (DataSize < 1)
				{
					UE_LOGF(LogNNERuntimeIREE, Error, "CPU module: vmfb data file '%ls' is empty", *InVmfbPath);
					return {};
				}

				void* Data = FMemory::Malloc(DataSize, IREE_HAL_HEAP_BUFFER_ALIGNMENT);
				Reader->Serialize(Data, DataSize);

				return MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(Data, DataSize, FMemory::Free), IREE_HAL_HEAP_BUFFER_ALIGNMENT);
			}

			static TSharedPtr<FModule> Make(const FString& InVmfbPath, TSharedRef<UE::NNE::FSharedModelData> ModelData, const FModuleMetaData& InModuleMetaData)
			{
				check(!InVmfbPath.IsEmpty());
				check(!InModuleMetaData.FunctionMetaData.IsEmpty());

				if (Modules.Contains(InVmfbPath) && Modules[InVmfbPath].IsValid())
				{
					return Modules[InVmfbPath].Pin();
				}

				TSharedPtr<FInstance> Instance = FInstance::GetInstance();
				if (!Instance.IsValid())
				{
					return TSharedPtr<FModule>();
				}

				iree_vm_module_t* Module = nullptr;
				if (!Instance->CreateModule(ModelData->GetView(), &Module))
				{
					return TSharedPtr<FModule>();
				}

				TSharedPtr<FModule> Result = TSharedPtr<FModule>(new FModule(Instance.ToSharedRef(), ModelData, Module, InModuleMetaData.FunctionMetaData));
				Modules.Add(InVmfbPath, Result);
				return Result;
			}

			bool AppendToSession(iree_runtime_session_t* InSession)
			{
				check(InSession);

				iree_status_t Status = iree_ok_status();
				check(iree_status_is_ok(Status));

				Status = iree_runtime_session_append_module(InSession, Module);
				if (!iree_status_is_ok(Status))
				{
					PrintIREEError("CPU module: Failed to append the module to the session", Status);
					iree_status_free(Status);
					return false;
				}

				iree_status_free(Status);
				return true;
			}

			TConstArrayView<UE::NNERuntimeIREE::FFunctionMetaData> GetFunctionMetaDataView()
			{
				return FunctionMetaData;
			}

			bool GetFunctionByName(const FString& InFunctionName, iree_vm_function_t* OutFunction)
			{
				iree_status_t Status = iree_ok_status();

				bool bFound = false;
				iree_host_size_t Ordinal = 0;
				iree_vm_function_t Function;
				do 
				{
					Status = iree_vm_module_lookup_function_by_ordinal(Module, IREE_VM_FUNCTION_LINKAGE_EXPORT, Ordinal, &Function);
					if (iree_status_is_ok(Status))
					{
						Ordinal++;
						iree_string_view_t FunctionNameView = iree_vm_function_name(&Function);
						FString FunctionName = FString::ConstructFromPtrSize(FunctionNameView.data, FunctionNameView.size);
						if (FunctionName.Equals(InFunctionName))
						{
							bFound = true;
							break;
						}
					}
				} while (iree_status_is_ok(Status));

				if (!bFound)
				{
					UE_LOGF(LogNNERuntimeIREE, Error, "CPU module: Failed to find the module function %ls", *InFunctionName);
					iree_status_free(Status);
					return false;
				}

				*OutFunction = Function;
				return true;
			}
		};
		TMap<FString, TWeakPtr<FModule>> FModule::Modules;
	} // Private

	namespace CPU
	{
		namespace Private
		{
			class FLibrary
			{
			private:
				void* Library;
				static TMap<FString, TWeakPtr<FLibrary>> Libraries;

				FLibrary(void* InLibrary) : Library(InLibrary)
				{
					check(InLibrary);
				}

			public:
				~FLibrary()
				{
					FPlatformProcess::FreeDllHandle(Library);
				}

				static TSharedPtr<FLibrary> GetLibrary(const FString& InLibraryPath, const FString& InLibraryName)
				{
					check(!InLibraryName.IsEmpty());

					FString CombinedPath = FPaths::Combine(InLibraryPath, InLibraryName);
					if (Libraries.Contains(CombinedPath) && Libraries[CombinedPath].IsValid())
					{
						return Libraries[CombinedPath].Pin();
					}

#ifdef NNE_RUNTIME_IREE_USE_COMBINED_LIB_PATH
					void* Library = FPlatformProcess::GetDllHandle(*CombinedPath);
					if (!Library)
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU library: Failed to load the shared library '%ls'", *CombinedPath);
						return TSharedPtr<FLibrary>();
					}
#else
					FPlatformProcess::PushDllDirectory(*InLibraryPath);
					void* Library = FPlatformProcess::GetDllHandle(*InLibraryName);
					FPlatformProcess::PopDllDirectory(*InLibraryPath);
					if (!Library)
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU library: Failed to load the shared library '%ls' from '%ls'", *InLibraryName, *InLibraryPath);
						return TSharedPtr<FLibrary>();
					}
#endif

					TSharedPtr<FLibrary> Result = TSharedPtr<FLibrary>(new FLibrary(Library));
					Libraries.Add(CombinedPath, Result);
					return Result;
				}

				bool GetFunctionPointer(const FString& InFunctionName, void** OutFunctionPointer) const
				{
					void* Result = FPlatformProcess::GetDllExport(Library, *InFunctionName);
					if (Result)
					{
						*OutFunctionPointer = Result;
						return true;
					}
					UE_LOGF(LogNNERuntimeIREE, Error, "CPU library: Failed to get the function %ls", *InFunctionName);
					return false;
				}
			};
			TMap<FString, TWeakPtr<FLibrary>> FLibrary::Libraries;

			class FDevice
			{
			private:
				TSharedRef<UE::NNERuntimeIREE::Private::FInstance> Instance;
				TSharedRef<FLibrary> Library;
				iree_hal_device_t* Device;
				static TMap<FString, TWeakPtr<FDevice>> Devices;

				FDevice(TSharedRef<UE::NNERuntimeIREE::Private::FInstance> InInstance, TSharedRef<FLibrary> InLibrary, iree_hal_device_t* InDevice) : Instance(InInstance), Library(InLibrary), Device(InDevice)
				{
					check(InDevice);
				}

			public:
				~FDevice()
				{
					iree_hal_device_release(Device);
				}

				static TSharedPtr<FDevice> Make(UE::NNERuntimeIREE::Private::FEnvironment& InEnvironment, const FString& InLibraryPath, const FString& InLibraryName, const FString& LibraryQueryFunctionName)
				{
					check(!InLibraryName.IsEmpty());
					check(!LibraryQueryFunctionName.IsEmpty());

					FString CombinedPathPlusFunction = FPaths::Combine(InLibraryPath, InLibraryName) + "::" + LibraryQueryFunctionName;
					if (Devices.Contains(CombinedPathPlusFunction) && Devices[CombinedPathPlusFunction].IsValid())
					{
						return Devices[CombinedPathPlusFunction].Pin();
					}

					TSharedPtr<FLibrary> Library = FLibrary::GetLibrary(InLibraryPath, InLibraryName);
					if (!Library.IsValid())
					{
						return TSharedPtr<FDevice>();
					}

					void* LibraryQueryFunctionPointer = nullptr;
					Library->GetFunctionPointer(LibraryQueryFunctionName, &LibraryQueryFunctionPointer);
					if (!LibraryQueryFunctionPointer)
					{
						return TSharedPtr<FDevice>();
					}

					TSharedPtr<UE::NNERuntimeIREE::Private::FInstance> Instance = UE::NNERuntimeIREE::Private::FInstance::GetInstance();
					if (!Instance.IsValid())
					{
						return TSharedPtr<FDevice>();
					}

					iree_hal_device_t* Device = nullptr;
					if (InEnvironment.GetConfig().ThreadingOptions.bIsSingleThreaded)
					{
						if (!Instance->CreateSyncDevice(LibraryQueryFunctionPointer, &Device))
						{
							return TSharedPtr<FDevice>();
						}
					}
					else
					{
						if (!Instance->CreateTaskDevice(InEnvironment, LibraryQueryFunctionPointer, &Device))
						{
							return TSharedPtr<FDevice>();
						}
					}

					check(Device);

					TSharedPtr<FDevice> Result = TSharedPtr<FDevice>(new FDevice(Instance.ToSharedRef(), Library.ToSharedRef(), Device));
					Devices.Add(CombinedPathPlusFunction, Result);
					return Result;
				}

				bool CreateSession(iree_runtime_session_t** OutSession)
				{
					return Instance->CreateSession(Device, OutSession);
				}

				iree_hal_allocator_t* GetDeviceAllocator()
				{
					return iree_hal_device_allocator(Device);
				}

				iree_allocator_t GetHostAllocator()
				{
					return iree_hal_device_host_allocator(Device);
				}

				iree_status_t CopyFromBuffer(iree_hal_buffer_t *Source, void *Target, iree_device_size_t CopySizeInBytes)
				{
					return iree_hal_device_transfer_d2h(Device, Source, 0, Target, CopySizeInBytes, IREE_HAL_TRANSFER_BUFFER_FLAG_DEFAULT, iree_infinite_timeout());
				}
			};
			TMap<FString, TWeakPtr<FDevice>> FDevice::Devices;

			class FSession
			{
			private:
				TSharedRef<FDevice> Device;
				TSharedRef<UE::NNERuntimeIREE::Private::FModule> Module;
				iree_runtime_session_t* Session;
				iree_runtime_call_t Call;
				TArray<UE::NNE::FTensorDesc> InputTensorDescs;
				TArray<UE::NNE::FTensorDesc> OutputTensorDescs;
				TArray<UE::NNE::FTensorShape> InputTensorShapes;
				TArray<UE::NNE::FTensorShape> OutputTensorShapes;
				TArray<UE::NNE::FTensorBindingCPU> PreviousInputBindings;

				FSession(TSharedRef<FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule, iree_runtime_session_t* InSession, const iree_runtime_call_t& InCall, TConstArrayView<UE::NNE::FTensorDesc> InInputTensorDescs, TConstArrayView<UE::NNE::FTensorDesc> InOutputTensorDescs)
					: Device(InDevice), Module(InModule), Session(InSession), Call(InCall), InputTensorDescs(InInputTensorDescs), OutputTensorDescs(InOutputTensorDescs)
				{
					check(InSession);
					check(!InputTensorDescs.IsEmpty());
					PreviousInputBindings.SetNum(InputTensorDescs.Num());
					iree_vm_list_resize(iree_runtime_call_inputs(&Call), InputTensorDescs.Num());
				}

			public:
				using ESetInputTensorShapesStatus = UE::NNE::IModelInstanceCPU::ESetInputTensorShapesStatus;
				using ERunSyncStatus = UE::NNE::IModelInstanceCPU::ERunSyncStatus;

				~FSession()
				{
					iree_runtime_call_deinitialize(&Call);
					iree_runtime_session_release(Session);
				}

				static TSharedPtr<FSession> Make(TSharedRef<FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule)
				{
					check(!InModule->GetFunctionMetaDataView().IsEmpty());

					iree_runtime_session_t* Session = nullptr;
					if (!InDevice->CreateSession(&Session))
					{
						return TSharedPtr<FSession>();
					}

					if (!InModule->AppendToSession(Session))
					{
						iree_runtime_session_release(Session);
						return TSharedPtr<FSession>();
					}

					iree_status_t Status = iree_ok_status();
					check(iree_status_is_ok(Status));

					FString MainFunctionName = InModule->GetFunctionMetaDataView()[0].Name;
					iree_vm_function_t MainFunction;
					if (!InModule->GetFunctionByName(MainFunctionName, &MainFunction))
					{
						iree_runtime_session_release(Session);
						iree_status_free(Status);
						return TSharedPtr<FSession>();
					}

					iree_host_size_t NumInputs = 0;
					iree_host_size_t NumOutputs = 0;
					iree_vm_function_signature_t Signature = iree_vm_function_signature(&MainFunction);
					Status = iree_vm_function_call_count_arguments_and_results(&Signature, &NumInputs, &NumOutputs);
					TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = InModule->GetFunctionMetaDataView()[0].InputDescs;
					TConstArrayView<UE::NNE::FTensorDesc> OutputTensorDescs = InModule->GetFunctionMetaDataView()[0].OutputDescs;
					if (!iree_status_is_ok(Status) || NumInputs != InputTensorDescs.Num() || NumOutputs != OutputTensorDescs.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Function signature mismatch in function %ls", *MainFunctionName);
						iree_runtime_session_release(Session);
						iree_status_free(Status);
						return TSharedPtr<FSession>();
					}

					iree_runtime_call_t Call;
					Status = iree_runtime_call_initialize(Session, MainFunction, &Call);
					if (!iree_status_is_ok(Status))
					{
						UE::NNERuntimeIREE::Private::PrintIREEError("CPU session: Failed to initialize the session call", Status);
						iree_runtime_session_release(Session);
						iree_status_free(Status);
						return TSharedPtr<FSession>();
					}

					TSharedPtr<FSession> Result = TSharedPtr<FSession>(new FSession(InDevice, InModule, Session, Call, InputTensorDescs, OutputTensorDescs));
					iree_status_free(Status);
					return Result;
				}

				TConstArrayView<UE::NNE::FTensorDesc> GetInputTensorDescs() const
				{
					return InputTensorDescs;
				}

				TConstArrayView<UE::NNE::FTensorDesc> GetOutputTensorDescs() const
				{
					return OutputTensorDescs;
				}

				TConstArrayView<UE::NNE::FTensorShape> GetInputTensorShapes() const
				{
					return InputTensorShapes;
				}

				TConstArrayView<UE::NNE::FTensorShape> GetOutputTensorShapes() const
				{
					return OutputTensorShapes;
				}

				ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<UE::NNE::FTensorShape> InInputShapes)
				{
					// OutputTensorShapes will be made available only if all shapes are concretes.
					OutputTensorShapes.Reset();
					bool bAllOutputShapeAreConcrete = true;
					for (int32 i = 0; i < OutputTensorDescs.Num(); i++)
					{
						bAllOutputShapeAreConcrete &= OutputTensorDescs[i].GetShape().IsConcrete();
					}
					if (bAllOutputShapeAreConcrete)
					{
						for (int32 i = 0; i < OutputTensorDescs.Num(); i++)
						{
							OutputTensorShapes.Add(UE::NNE::FTensorShape::MakeFromSymbolic(OutputTensorDescs[i].GetShape()));
						}
					}

					InputTensorShapes.Reset(InInputShapes.Num());
					if (InInputShapes.Num() != InputTensorDescs.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Number of input shapes does not match number of input tensors");
						return ESetInputTensorShapesStatus::Fail;
					}

					for (int32 i = 0; i < InInputShapes.Num(); ++i)
					{
						const UE::NNE::FTensorDesc SymbolicDesc = InputTensorDescs[i];
						if (!InInputShapes[i].IsCompatibleWith(SymbolicDesc.GetShape()))
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Input shape does not match input tensor %ls of index %d", *SymbolicDesc.GetName(), i);
							return ESetInputTensorShapesStatus::Fail;
						}
					}
					InputTensorShapes = InInputShapes;

					return ESetInputTensorShapesStatus::Ok;
				}

				ERunSyncStatus RunSyncCPU(TConstArrayView<UE::NNE::FTensorBindingCPU> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingCPU> InOutputBindings)
				{
					LLM_SCOPE_BYTAG(NNERuntimeIREE_Cpu);
					SCOPED_NAMED_EVENT_TEXT("NNERuntimeIREE::CPU::RunSync", FColor::Magenta);
					// Verify the model inputs were prepared
					if (InputTensorShapes.IsEmpty())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Input shapes are not set, please call SetInputTensorShapes.");
						return ERunSyncStatus::Fail;
					}
					if (InInputBindings.Num() != InputTensorShapes.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Input bindings need to match input tensor descriptor count (got %d, expected %d).", InInputBindings.Num(), InputTensorShapes.Num());
						return ERunSyncStatus::Fail;
					}
					if (!InOutputBindings.IsEmpty() && InOutputBindings.Num() != OutputTensorDescs.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Output binding can be empty or needs to match output tensor descriptor count (got %d, expected %d).", InOutputBindings.Num(), OutputTensorDescs.Num());
						return ERunSyncStatus::Fail;
					}
					
					for (int32 i = 0; i < InInputBindings.Num(); i++)
					{
						if (!InInputBindings[i].Data && InInputBindings[i].SizeInBytes != 0)
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Binding input tensor %d is not set but given size is non-zero %lld.", i, InInputBindings[i].SizeInBytes);
							return ERunSyncStatus::Fail;
						}
						if (InInputBindings[i].SizeInBytes != (InputTensorShapes[i].Volume() * InputTensorDescs[i].GetElementByteSize()))
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Binding input tensor %d size does not match size given by tensor descriptor (got %lld, expected %lld).", i, InInputBindings[i].SizeInBytes, InputTensorShapes[i].Volume() * InputTensorDescs[i].GetElementByteSize());
							return ERunSyncStatus::Fail;
						}
						if (FMath::Modulo<uint64>((uint64)InInputBindings[i].Data, IREE_HAL_HEAP_BUFFER_ALIGNMENT) != 0)
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Input bindings memory need to be aligned with %d bytes", IREE_HAL_HEAP_BUFFER_ALIGNMENT);
							return ERunSyncStatus::Fail;
						}
					}

					for (int32 i = 0; i < InOutputBindings.Num(); i++)
					{
						if (InOutputBindings[i].Data && FMath::Modulo<uint64>((uint64)InOutputBindings[i].Data, IREE_HAL_HEAP_BUFFER_ALIGNMENT) != 0)
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "CPU session: Output bindings memory need to be aligned with %d bytes", IREE_HAL_HEAP_BUFFER_ALIGNMENT);
							return ERunSyncStatus::Fail;
						}
					}

					iree_status_t Status = RunSyncCPUInternal(InInputBindings, InOutputBindings);

					if (!iree_status_is_ok(Status))
					{
						UE::NNERuntimeIREE::Private::PrintIREEError("CPU session: RunSyncCPU failed", Status);
						iree_status_free(Status);
						return ERunSyncStatus::Fail;
					}
					return ERunSyncStatus::Ok;
				}

			private:

				iree_status_t RunSyncCPUInternal(TConstArrayView<UE::NNE::FTensorBindingCPU> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingCPU> InOutputBindings)
				{
					iree_hal_allocator_t *DeviceAllocator = Device->GetDeviceAllocator();
					iree_allocator_t HostAllocator = Device->GetHostAllocator();
					iree_vm_list_t *Inputs = iree_runtime_call_inputs(&Call);
					iree_vm_list_t *Outputs = iree_runtime_call_outputs(&Call);
					check(PreviousInputBindings.Num() == InInputBindings.Num());
					check(iree_vm_list_size(Inputs) == InInputBindings.Num());
					for (int32 i = 0; i < InInputBindings.Num(); i++)
					{
						//Create a new input buffer view if the input binding is different
						//thant that of the previous call
						if (PreviousInputBindings[i].Data != InInputBindings[i].Data ||
							PreviousInputBindings[i].SizeInBytes != InInputBindings[i].SizeInBytes)
						{
							//Only set previous input binding once we successfully added 
							//the input binding to the Inputs list
							// These flags are added when creating a heap buffer. Source: iree\hal\allocator_heap.c:162
							const iree_hal_memory_type_t HEAP_MEMORY_TYPE = IREE_HAL_MEMORY_TYPE_HOST_VISIBLE;
							const iree_hal_buffer_usage_t HEAP_BUFFER_USAGE =
								IREE_HAL_BUFFER_USAGE_MAPPING_SCOPED |
								IREE_HAL_BUFFER_USAGE_MAPPING_PERSISTENT |
								IREE_HAL_BUFFER_USAGE_MAPPING_ACCESS_RANDOM |
								IREE_HAL_BUFFER_USAGE_TRANSFER;

							iree_hal_buffer_t* Buffer = nullptr;
							IREE_RETURN_IF_ERROR(
								iree_hal_heap_buffer_wrap(iree_hal_buffer_placement_undefined(),
														  IREE_HAL_MEMORY_TYPE_DEVICE_LOCAL | HEAP_MEMORY_TYPE,
														  IREE_HAL_MEMORY_ACCESS_READ,
														  IREE_HAL_BUFFER_USAGE_DEFAULT | HEAP_BUFFER_USAGE,
														  InInputBindings[i].SizeInBytes,
														  iree_make_byte_span(InInputBindings[i].Data, InInputBindings[i].SizeInBytes),
														  iree_hal_buffer_release_callback_null(),
														  HostAllocator,
														  &Buffer),
								"heap buffer wrap failed");

							iree_hal_dim_t Shape[UE::NNE::FTensorShape::MaxRank];
							for (int32 j = 0; j < InputTensorShapes[i].Rank(); j++)
							{
								Shape[j] = InputTensorShapes[i].GetData()[j];
							}
							ENNETensorDataType NNEType = InputTensorDescs[i].GetDataType();
							iree_hal_element_types_t IREEType = UE::NNERuntimeIREE::Private::NNEToIREEType(NNEType);
							iree_hal_buffer_view_t* BufferView = nullptr;
							IREE_RETURN_AND_EVAL_IF_ERROR(iree_hal_buffer_release(Buffer),
								iree_hal_buffer_view_create(Buffer, InputTensorShapes[i].Rank(), Shape,
															IREEType, IREE_HAL_ENCODING_TYPE_DENSE_ROW_MAJOR,
															HostAllocator, &BufferView),
								"creation of input BufferView failed");
							//Now that BufferView also has ownership of the Buffer we can release our ownership
							iree_hal_buffer_release(Buffer);

							iree_vm_ref_t BufferViewRef = {0};
							//Transfers ownership from BufferView to BufferViewRef
							IREE_RETURN_AND_EVAL_IF_ERROR(iree_hal_buffer_view_release(BufferView),
								iree_vm_ref_wrap_assign(BufferView, iree_hal_buffer_view_type(), &BufferViewRef),
								"wrap assignment of BufferView failed");
							//Releases previous element at i and transfers ownership from BufferViewRef to Inputs
  							IREE_RETURN_AND_EVAL_IF_ERROR(iree_vm_ref_release(&BufferViewRef),
								iree_vm_list_set_ref_move(Inputs, i, &BufferViewRef),
								"set BufferView to input list failed");
							PreviousInputBindings[i] = InInputBindings[i];
						}
					}
					//Releases all elements and sets count to 0. Doesn't release the lists memory
					//This is needed in case the previous call failed to pop all elements
					iree_vm_list_clear(Outputs);

					IREE_RETURN_IF_ERROR(
						iree_runtime_call_invoke(&Call, 0),
						"UE::NNERuntimeIREE::CPU::Private::FSession failed to call the model function");

					check(iree_vm_list_size(Outputs) == OutputTensorDescs.Num());
					check(OutputTensorShapes.Num() == 0 || OutputTensorShapes.Num() == OutputTensorDescs.Num());
					if (OutputTensorShapes.Num() == 0 && OutputTensorDescs.Num() > 0)
					{
						OutputTensorShapes.SetNum(OutputTensorDescs.Num());
					}
					check(OutputTensorShapes.Num() == OutputTensorDescs.Num());
					check(InOutputBindings.IsEmpty() || InOutputBindings.Num() == OutputTensorDescs.Num());

					for (int32 i = 0; i < OutputTensorDescs.Num(); i++)
					{
						iree_hal_buffer_view_t* BufferView = nullptr;
						IREE_RETURN_IF_ERROR(
							iree_runtime_call_outputs_pop_front_buffer_view(&Call, &BufferView),
							"failed to get output at index %d", i);
						iree_host_size_t Rank = iree_hal_buffer_view_shape_rank(BufferView);
						const iree_hal_dim_t* Dims = iree_hal_buffer_view_shape_dims(BufferView);
						if (!UE::NNERuntimeIREE::Private::ShapeEqualDims(OutputTensorShapes[i], Dims, Rank))
						{
							OutputTensorShapes[i] = UE::NNERuntimeIREE::Private::DimsToShape(Dims, Rank);
						}

						const uint64 DataSizeInBytes = iree_hal_buffer_view_byte_length(BufferView);

						// Copy result to output binding if present and buffer big enough
						// note: by design silently skip if output buffer is undersized.
						if (!InOutputBindings.IsEmpty() && InOutputBindings[i].Data && DataSizeInBytes <= InOutputBindings[i].SizeInBytes)
						{
							iree_hal_buffer_t* Buffer = iree_hal_buffer_view_buffer(BufferView);
							if (!Buffer)
							{
								iree_hal_buffer_view_release(BufferView);
								return iree_make_status(IREE_STATUS_UNKNOWN, "Failed to get the result buffer");
							}

							IREE_RETURN_AND_EVAL_IF_ERROR(iree_hal_buffer_view_release(BufferView),
								Device->CopyFromBuffer(Buffer, InOutputBindings[i].Data, DataSizeInBytes),
								"Copy to the output buffer failed");
						}
						iree_hal_buffer_view_release(BufferView);
					}
					return iree_ok_status();
				}
			};
		}

		FModelInstance::FModelInstance(TSharedRef<UE::NNERuntimeIREE::CPU::Private::FSession> InSession) : Session(InSession)
		{
			
		}

		TSharedPtr<FModelInstance> FModelInstance::Make(TSharedRef<UE::NNERuntimeIREE::CPU::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule)
		{
			TSharedPtr<Private::FSession> Session = Private::FSession::Make(InDevice, InModule);
			if (!Session.IsValid())
			{
				return TSharedPtr<FModelInstance>();
			}

			return TSharedPtr<FModelInstance>(new FModelInstance(Session.ToSharedRef()));
		}

		TConstArrayView<UE::NNE::FTensorDesc> FModelInstance::GetInputTensorDescs() const
		{
			return Session->GetInputTensorDescs();
		}

		TConstArrayView<UE::NNE::FTensorDesc> FModelInstance::GetOutputTensorDescs() const
		{
			return Session->GetOutputTensorDescs();
		}

		TConstArrayView<UE::NNE::FTensorShape> FModelInstance::GetInputTensorShapes() const
		{
			return Session->GetInputTensorShapes();
		}

		TConstArrayView<UE::NNE::FTensorShape> FModelInstance::GetOutputTensorShapes() const
		{
			return Session->GetOutputTensorShapes();
		}

		FModelInstance::ESetInputTensorShapesStatus FModelInstance::SetInputTensorShapes(TConstArrayView<UE::NNE::FTensorShape> InInputShapes)
		{
			return Session->SetInputTensorShapes(InInputShapes);
		}

		FModelInstance::ERunSyncStatus FModelInstance::RunSync(TConstArrayView<UE::NNE::FTensorBindingCPU> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingCPU> InOutputBindings)
		{
			return Session->RunSyncCPU(InInputBindings, InOutputBindings);
		}

		FModel::FModel(TSharedRef<UE::NNERuntimeIREE::CPU::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule) : Device(InDevice), Module(InModule)
		{

		}

		TSharedPtr<FModel> FModel::Make(UE::NNERuntimeIREE::Private::FEnvironment& InEnvironment, const FString& InDirPath, const FString& InSharedLibraryFileName, const FString& InVmfbFileName, const FString& InLibraryQueryFunctionName, const FModuleMetaData& InModuleMetaData)
		{
			check(!InSharedLibraryFileName.IsEmpty());
			check(!InVmfbFileName.IsEmpty());
			check(!InLibraryQueryFunctionName.IsEmpty());
			check(!InModuleMetaData.FunctionMetaData.IsEmpty());

			TSharedPtr<Private::FDevice> Device = Private::FDevice::Make(InEnvironment, InDirPath, InSharedLibraryFileName, InLibraryQueryFunctionName);
			if (!Device.IsValid())
			{
				return {};
			}

			const FString CombinedPath = FPaths::Combine(InDirPath, InVmfbFileName);
			const TSharedPtr<UE::NNE::FSharedModelData> ModelData = UE::NNERuntimeIREE::Private::FModule::GetVmfbAsModelData(CombinedPath);
			if (!ModelData.IsValid())
			{
				return {};
			}

			TSharedPtr<UE::NNERuntimeIREE::Private::FModule> Module = UE::NNERuntimeIREE::Private::FModule::Make(CombinedPath, ModelData.ToSharedRef(), InModuleMetaData);
			if (!Module.IsValid())
			{
				return {};
			}

			return TSharedPtr<FModel>(new FModel(Device.ToSharedRef(), Module.ToSharedRef()));
		}

		TSharedPtr<UE::NNE::IModelInstanceCPU> FModel::CreateModelInstanceCPU()
		{
			return FModelInstance::Make(Device, Module);
		}
	} // CPU

#ifdef WITH_NNE_RUNTIME_IREE_RDG
	namespace RDG
	{
		namespace Private
		{
			class FDevice
			{
				FDevice(TSharedRef<UE::NNERuntimeIREE::Private::FInstance> InInstance, iree_hal_device_t* InDevice) : Instance(InInstance), Device(InDevice)
				{
					check(InDevice);
				}

			public:
				~FDevice()
				{
					iree_hal_device_release(Device);
				}

				static TSharedPtr<FDevice> Make(const TMap<FString, TConstArrayView<uint8>>& Executables)
				{
					TSharedPtr<UE::NNERuntimeIREE::Private::FInstance> Instance = UE::NNERuntimeIREE::Private::FInstance::GetInstance();
					if (!Instance.IsValid())
					{
						return {};
					}

					iree_string_view_t Identifier = iree_make_cstring_view("unreal");

					iree_hal_device_t* Device = nullptr;
					iree_status_t Status = IREE::HAL::RDG::DeviceCreate(Identifier, Instance->GetHostAllocator(), Executables, &Device);
					if (!iree_status_is_ok(Status))
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "Could not create IREE RDG device!");
						iree_status_free(Status);
						return {};
					}

					TSharedPtr<FDevice> Result = TSharedPtr<FDevice>(new FDevice(Instance.ToSharedRef(), Device));
					
					Devices.Add("RDG", Result);
					
					return Result;
				}

				bool CreateSession(iree_runtime_session_t** OutSession)
				{
					return Instance->CreateSession(Device, OutSession);
				}

				iree_hal_allocator_t* GetDeviceAllocator() const
				{
					return iree_hal_device_allocator(Device);
				}

				iree_allocator_t GetHostAllocator()
				{
					return iree_hal_device_host_allocator(Device);
				}

			private:
				TSharedRef<UE::NNERuntimeIREE::Private::FInstance> Instance;
				iree_hal_device_t* Device;
				static TMap<FString, TWeakPtr<FDevice>> Devices;
			};
			TMap<FString, TWeakPtr<FDevice>> FDevice::Devices;

			class FGraphBuilderSetter
			{
			public:
				FGraphBuilderSetter(TSharedRef<FDevice> Device, FRDGBuilder& GraphBuilder) : Device(Device)
				{
					IREE::HAL::RDG::DeviceAllocatorSetGraphBuilder(Device->GetDeviceAllocator(), GraphBuilder);
				}

				~FGraphBuilderSetter()
				{
					IREE::HAL::RDG::DeviceAllocatorResetGraphBuilder(Device->GetDeviceAllocator());
				}

			private:
				TSharedRef<FDevice> Device;
			};
			class FSession
			{
			public:
				using ESetInputTensorShapesStatus = UE::NNE::IModelInstanceRDG::ESetInputTensorShapesStatus;
				using EEnqueueRDGStatus = UE::NNE::IModelInstanceRDG::EEnqueueRDGStatus;

				~FSession()
				{
					iree_runtime_call_deinitialize(&Call);
					iree_runtime_session_release(Session);
				}

				static TSharedPtr<FSession> Make(TSharedRef<FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule)
				{
					check(!InModule->GetFunctionMetaDataView().IsEmpty());

					iree_runtime_session_t* Session = nullptr;
					if (!InDevice->CreateSession(&Session))
					{
						return {};
					}

					{
						FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);
						bool bAppendToSessionSuccess = true;

						ENQUEUE_RENDER_COMMAND(FModelInstanceRDG_Run)
						(
							[Signal, &bAppendToSessionSuccess, &InModule, &Session, &InDevice](FRHICommandListImmediate& RHICmdList)
							{
								FRDGBuilder	GraphBuilder(RHICmdList);
								
								{
									FGraphBuilderSetter GraphBuilderSetter(InDevice, GraphBuilder);

									if (!InModule->AppendToSession(Session))
									{
										iree_runtime_session_release(Session);
										bAppendToSessionSuccess = false;
									}
								}

								GraphBuilder.Execute();

								RHICmdList.SubmitAndBlockUntilGPUIdle();
								Signal->Trigger();
							}
						);

						Signal->Wait();

						if (!bAppendToSessionSuccess)
						{
							return {};
						}
					}

					FString MainFunctionName = InModule->GetFunctionMetaDataView()[0].Name;
					iree_vm_function_t MainFunction;
					if (!InModule->GetFunctionByName(MainFunctionName, &MainFunction))
					{
						iree_runtime_session_release(Session);
						return {};
					}

					iree_host_size_t NumInputs = 0;
					iree_host_size_t NumOutputs = 0;
					iree_vm_function_signature_t Signature = iree_vm_function_signature(&MainFunction);
					iree_status_t Status = iree_vm_function_call_count_arguments_and_results(&Signature, &NumInputs, &NumOutputs);
					TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = InModule->GetFunctionMetaDataView()[0].InputDescs;
					TConstArrayView<UE::NNE::FTensorDesc> OutputTensorDescs = InModule->GetFunctionMetaDataView()[0].OutputDescs;
					if (!iree_status_is_ok(Status) || NumInputs != InputTensorDescs.Num() || NumOutputs != OutputTensorDescs.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Function signature mismatch in function %ls", *MainFunctionName);
						iree_runtime_session_release(Session);
						iree_status_free(Status);
						return {};
					}

					iree_runtime_call_t Call;
					Status = iree_runtime_call_initialize(Session, MainFunction, &Call);
					if (!iree_status_is_ok(Status))
					{
						UE::NNERuntimeIREE::Private::PrintIREEError("RDG session: Failed to initialize the session call", Status);
						iree_runtime_session_release(Session);
						iree_status_free(Status);
						return {};
					}

					TSharedPtr<FSession> Result = TSharedPtr<FSession>(new FSession(InDevice, InModule, Session, Call, InputTensorDescs, OutputTensorDescs));
					iree_status_free(Status);
					return Result;
				}

				TConstArrayView<UE::NNE::FTensorDesc> GetInputTensorDescs() const
				{
					return InputTensorDescs;
				}

				TConstArrayView<UE::NNE::FTensorDesc> GetOutputTensorDescs() const
				{
					return OutputTensorDescs;
				}

				TConstArrayView<UE::NNE::FTensorShape> GetInputTensorShapes() const
				{
					return InputTensorShapes;
				}

				TConstArrayView<UE::NNE::FTensorShape> GetOutputTensorShapes() const
				{
					return OutputTensorShapes;
				}

				ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<UE::NNE::FTensorShape> InInputShapes)
				{
					// OutputTensorShapes will be made available only if all shapes are concretes.
					OutputTensors.Reset();
					OutputTensorShapes.Reset();
					bool bAllOutputShapeAreConcrete = true;
					for (int32 i = 0; i < OutputTensorDescs.Num(); i++)
					{
						bAllOutputShapeAreConcrete &= OutputTensorDescs[i].GetShape().IsConcrete();
					}
					if (bAllOutputShapeAreConcrete)
					{
						for (int32 i = 0; i < OutputTensorDescs.Num(); i++)
						{
							NNERuntimeIREE::Private::FTensor Tensor = NNERuntimeIREE::Private::FTensor::MakeFromSymbolicDesc(OutputTensorDescs[i]);
							OutputTensors.Add(Tensor);
							OutputTensorShapes.Add(Tensor.GetShape());
						}
					}

					InputTensors.Reset();
					InputTensorShapes.Reset(InInputShapes.Num());
					if (InInputShapes.Num() != InputTensorDescs.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Number of input shapes does not match number of input tensors");
						return ESetInputTensorShapesStatus::Fail;
					}

					for (int32 i = 0; i < InInputShapes.Num(); ++i)
					{
						const UE::NNE::FTensorDesc SymbolicDesc = InputTensorDescs[i];
						if (!InInputShapes[i].IsCompatibleWith(SymbolicDesc.GetShape()))
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Input shape does not match input tensor %ls of index %d", *SymbolicDesc.GetName(), i);
							return ESetInputTensorShapesStatus::Fail;
						}

						NNERuntimeIREE::Private::FTensor Tensor = NNERuntimeIREE::Private::FTensor::Make(InInputShapes[i], SymbolicDesc.GetDataType());
						InputTensors.Emplace(Tensor);
					}
					InputTensorShapes = InInputShapes;

					return ESetInputTensorShapesStatus::Ok;
				}

				EEnqueueRDGStatus EnqueueRDG(FRDGBuilder& GraphBuilder, TConstArrayView<UE::NNE::FTensorBindingRDG> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingRDG> InOutputBindings)
				{
					SCOPED_NAMED_EVENT_TEXT("NNERuntimeIREE::RDG::EnqueueRDG", FColor::Magenta);

					RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNERuntimeIREERDGEnqueue, "NNERuntimeIREERdg.EnqueueRDG");

					// Verify the model inputs were prepared
					if (InputTensorShapes.IsEmpty())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Input shapes are not set, please call SetInputTensorShapes.");
						return EEnqueueRDGStatus::Fail;
					}

					if (InInputBindings.Num() != InputTensorShapes.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Input bindings need to match input tensor descriptor count (got %d, expected %d).", InInputBindings.Num(), InputTensorShapes.Num());
						return EEnqueueRDGStatus::Fail;
					}

					if (!InOutputBindings.IsEmpty() && InOutputBindings.Num() != OutputTensorDescs.Num())
					{
						UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Output binding can be empty or needs to match output tensor descriptor count (got %d, expected %d).", InOutputBindings.Num(), OutputTensorDescs.Num());
						return EEnqueueRDGStatus::Fail;
					}

					for (int32 i = 0; i < InInputBindings.Num(); i++)
					{
						const NNE::FTensorBindingRDG& Binding = InInputBindings[i];
						if (!Binding.Buffer)
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Binding input tensor %d is null.", i);
							return EEnqueueRDGStatus::Fail;
						}

						if (Binding.Buffer->Desc.GetSize() < InputTensors[i].GetDataSize())
						{
							UE_LOGF(LogNNERuntimeIREE, Error, "RDG session: Binding input tensor %d size (%d) is smaller than the size (%lld) given by tensor descriptor.", i, Binding.Buffer->Desc.GetSize(), InputTensorShapes[i].Volume() * InputTensorDescs[i].GetElementByteSize());
							return EEnqueueRDGStatus::Fail;
						}
					}

					iree_runtime_call_reset(&Call);

					iree_status_t Status = iree_ok_status();

					FGraphBuilderSetter GraphBuilderSetter(Device, GraphBuilder);

					const iree_hal_memory_type_t MemoryType = IREE_HAL_MEMORY_TYPE_DEVICE_LOCAL;
					const iree_hal_memory_access_t BufferAccess = IREE_HAL_MEMORY_ACCESS_ALL;
					const iree_hal_buffer_usage_t BufferUsage = IREE_HAL_BUFFER_USAGE_TRANSFER | IREE_HAL_BUFFER_USAGE_DISPATCH_STORAGE;

					TArray<iree_hal_buffer_t*> IREEInputBuffers;
					for (int i = 0; i < InInputBindings.Num(); i++)
					{
						iree_hal_buffer_t* IREEBuffer = NULL;
						if (iree_status_is_ok(Status))
						{
							Status = IREE::HAL::RDG::BufferWrapRDG(
								Device->GetHostAllocator(),
								Device->GetDeviceAllocator(),
								MemoryType,
								BufferAccess,
								BufferUsage,
								InInputBindings[i].Buffer->Desc.GetSize(),
								0,
								InInputBindings[i].Buffer->Desc.GetSize(),
								&GraphBuilder,
								InInputBindings[i].Buffer,
								iree_hal_buffer_release_callback_null(),
								&IREEBuffer);
						}
						if (iree_status_is_ok(Status))
						{
							IREEInputBuffers.Add(IREEBuffer);
						}

						TArray<iree_hal_dim_t> IREEShape;
						for (uint32 Dim : InputTensorShapes[i].GetData())
						{
							IREEShape.Add(Dim);
						}

						iree_hal_buffer_view_t* IREEBufferView = NULL;
						if (iree_status_is_ok(Status))
						{
							Status = iree_hal_buffer_view_create(
								IREEBuffer,
								IREEShape.Num(), IREEShape.GetData(),
								UE::NNERuntimeIREE::Private::NNEToIREEType(InputTensorDescs[i].GetDataType()),
								IREE_HAL_ENCODING_TYPE_DENSE_ROW_MAJOR,
								Device->GetHostAllocator(),
								&IREEBufferView);
						}

						if (iree_status_is_ok(Status))
						{
							Status = iree_runtime_call_inputs_push_back_buffer_view(&Call, IREEBufferView);
							iree_hal_buffer_view_release(IREEBufferView);
						}

						if (!iree_status_is_ok(Status))
						{
							break;
						}
					}

					// The buffers are retained by the view, if actually created...
					// ...but in any case we have to release them.
					for (iree_hal_buffer_t* IREEBuffer : IREEInputBuffers)
					{
						iree_hal_buffer_release(IREEBuffer);
					}

					if (iree_status_is_ok(Status))
					{
						RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNERuntimeIREERDGCall, "NNERuntimeIREERdg.EnqueueRDG.Call");

						UE_LOGF(LogNNERuntimeIREE, Verbose, "NNERuntimeIREERDG::FSession::EnqueueRDG: synchronous invocation of IREE call.");

						Status = iree_runtime_call_invoke(&Call, 0);
					}

					if (iree_status_is_ok(Status))
					{
						check(OutputTensorShapes.Num() == 0 || OutputTensorShapes.Num() == OutputTensorDescs.Num());
						bool bSetOutputTensorShapes = false;
						if (OutputTensorShapes.Num() == 0 && OutputTensorDescs.Num() > 0)
						{
							bSetOutputTensorShapes = true;
							OutputTensorShapes.SetNum(OutputTensorDescs.Num());
						}
						check(OutputTensorShapes.Num() == OutputTensorDescs.Num());
						check(InOutputBindings.IsEmpty() || InOutputBindings.Num() == OutputTensorDescs.Num());

						for (int32 i = 0; i < OutputTensorDescs.Num(); i++)
						{
							iree_hal_buffer_view_t* IREEBufferView = nullptr;
							Status = iree_runtime_call_outputs_pop_front_buffer_view(&Call, &IREEBufferView);
							if (!iree_status_is_ok(Status))
							{
								if (bSetOutputTensorShapes)
								{
									OutputTensorShapes.Reset();
								}
								break;
							}
							iree_host_size_t Rank = iree_hal_buffer_view_shape_rank(IREEBufferView);
							const iree_hal_dim_t* Dims = iree_hal_buffer_view_shape_dims(IREEBufferView);
							if (!UE::NNERuntimeIREE::Private::ShapeEqualDims(OutputTensorShapes[i], Dims, Rank))
							{
								OutputTensorShapes[i] = UE::NNERuntimeIREE::Private::DimsToShape(Dims, Rank);
							}

							if (InOutputBindings.IsEmpty() || !InOutputBindings[i].Buffer)
							{
								iree_hal_buffer_view_release(IREEBufferView);
								continue;
							}

							iree_hal_buffer_t* IREEBuffer = iree_hal_buffer_view_buffer(IREEBufferView);
							if (!IREEBuffer)
							{
								Status = iree_make_status(IREE_STATUS_UNKNOWN, "Failed to get the result buffer");
								iree_hal_buffer_view_release(IREEBufferView);
								break;
							}

							const uint64 Length = (uint64)iree_hal_buffer_view_byte_length(IREEBufferView);
							const uint64 Offset = (uint64)iree_hal_buffer_byte_offset(IREEBuffer);

							FRDGBufferRef RDGBuffer = IREE::HAL::RDG::BufferRDGBuffer(IREEBuffer, &GraphBuilder);
							check(Length <= RDGBuffer->Desc.GetSize() - Offset);

							// note: by design silently skip if output buffer is undersized.
							if (Length <= (uint64)InOutputBindings[i].Buffer->Desc.GetSize())
							{
								AddCopyBufferPass(GraphBuilder, InOutputBindings[i].Buffer, 0, RDGBuffer, Offset, Length);
							}

							iree_hal_buffer_view_release(IREEBufferView);
						}
					}

					if (!iree_status_is_ok(Status))
					{
						NNERuntimeIREE::Private::PrintIREEError("EnqueueRDG Failed!", Status);
					}

					iree_status_free(Status);
					return iree_status_is_ok(Status) ? EEnqueueRDGStatus::Ok : EEnqueueRDGStatus::Fail;
				}

			private:
				TSharedRef<FDevice> Device;
				TSharedRef<UE::NNERuntimeIREE::Private::FModule> Module;
				iree_runtime_session_t* Session;
				iree_runtime_call_t Call;
				TArray<UE::NNE::FTensorDesc> InputTensorDescs;
				TArray<UE::NNE::FTensorDesc> OutputTensorDescs;
				TArray<UE::NNE::FTensorShape> InputTensorShapes;
				TArray<UE::NNE::FTensorShape> OutputTensorShapes;
				TArray<NNERuntimeIREE::Private::FTensor> InputTensors;
				TArray<NNERuntimeIREE::Private::FTensor> OutputTensors;

				FSession(TSharedRef<FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule, iree_runtime_session_t* InSession, const iree_runtime_call_t& InCall, TConstArrayView<UE::NNE::FTensorDesc> InInputTensorDescs, TConstArrayView<UE::NNE::FTensorDesc> InOutputTensorDescs)
					: Device(InDevice), Module(InModule), Session(InSession), Call(InCall), InputTensorDescs(InInputTensorDescs), OutputTensorDescs(InOutputTensorDescs)
				{
					check(InSession);
					check(!InputTensorDescs.IsEmpty());
				}
			};

		} // namespace Private

		FModelInstance::FModelInstance(TSharedRef<UE::NNERuntimeIREE::RDG::Private::FSession> InSession) : Session(InSession)
		{
			
		}

		TSharedPtr<FModelInstance> FModelInstance::Make(TSharedRef<UE::NNERuntimeIREE::RDG::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule)
		{
			TSharedPtr<Private::FSession> Session = Private::FSession::Make(InDevice, InModule);
			if (!Session.IsValid())
			{
				return TSharedPtr<FModelInstance>();
			}

			return TSharedPtr<FModelInstance>(new FModelInstance(Session.ToSharedRef()));
		}

		TConstArrayView<UE::NNE::FTensorDesc> FModelInstance::GetInputTensorDescs() const
		{
			return Session->GetInputTensorDescs();
		}

		TConstArrayView<UE::NNE::FTensorDesc> FModelInstance::GetOutputTensorDescs() const
		{
			return Session->GetOutputTensorDescs();
		}

		TConstArrayView<UE::NNE::FTensorShape> FModelInstance::GetInputTensorShapes() const
		{
			return Session->GetInputTensorShapes();
		}

		TConstArrayView<UE::NNE::FTensorShape> FModelInstance::GetOutputTensorShapes() const
		{
			return Session->GetOutputTensorShapes();
		}

		FModelInstance::ESetInputTensorShapesStatus FModelInstance::SetInputTensorShapes(TConstArrayView<UE::NNE::FTensorShape> InInputShapes)
		{
			return Session->SetInputTensorShapes(InInputShapes);
		}

		FModelInstance::EEnqueueRDGStatus FModelInstance::EnqueueRDG(FRDGBuilder& GraphBuilder, TConstArrayView<UE::NNE::FTensorBindingRDG> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingRDG> InOutputBindings)
		{
			return Session->EnqueueRDG(GraphBuilder, InInputBindings, InOutputBindings);
		}

		FModel::FModel(TSharedRef<UE::NNERuntimeIREE::RDG::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule) : Device(InDevice), Module(InModule)
		{

		}

		TSharedPtr<FModel> FModel::Make(const FString& InDirPath, TConstArrayView64<uint8> VmfbData, const FModuleMetaData& InModuleMetaData, const TMap<FString, TConstArrayView<uint8>>& Executables)
		{
			check(!InDirPath.IsEmpty());
			check(!VmfbData.IsEmpty());
			check(!InModuleMetaData.FunctionMetaData.IsEmpty());

			TSharedPtr<Private::FDevice> Device = Private::FDevice::Make(Executables);
			if (!Device.IsValid())
			{
				return {};
			}

			const int64 DataSize = VmfbData.Num();

			void* Data = FMemory::Malloc(DataSize, IREE_HAL_HEAP_BUFFER_ALIGNMENT);
			FMemory::Memcpy(Data, VmfbData.GetData(), DataSize);

			TSharedPtr<UE::NNE::FSharedModelData> ModelData = MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(Data, DataSize, FMemory::Free), IREE_HAL_HEAP_BUFFER_ALIGNMENT);
			if (!ModelData.IsValid())
			{
				return {};
			}

			TSharedPtr<UE::NNERuntimeIREE::Private::FModule> Module = UE::NNERuntimeIREE::Private::FModule::Make(InDirPath, ModelData.ToSharedRef(), InModuleMetaData);
			if (!Module.IsValid())
			{
				return {};
			}

			return TSharedPtr<FModel>(new FModel(Device.ToSharedRef(), Module.ToSharedRef()));
		}

		TSharedPtr<UE::NNE::IModelInstanceRDG> FModel::CreateModelInstanceRDG()
		{
			return FModelInstance::Make(Device, Module);
		}

	} // namespace RDG
#endif // WITH_NNE_RUNTIME_IREE_RDG
} // UE::NNERuntimeIREE

#endif // WITH_NNE_RUNTIME_IREE
