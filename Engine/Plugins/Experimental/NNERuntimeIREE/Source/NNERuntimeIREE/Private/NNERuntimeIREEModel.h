// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE

#include "NNERuntimeCPU.h"
#ifdef WITH_NNE_RUNTIME_IREE_RDG
#include "NNERuntimeRDG.h"
#endif // WITH_NNE_RUNTIME_IREE_RDG
#include "NNERuntimeIREEMetaData.h"

LLM_DECLARE_TAG(NNERuntimeIREE_Cpu);

namespace UE::NNERuntimeIREE
{
	namespace Private
	{
		class FEnvironment;
		class FModule;
	} // Private

	namespace CPU
	{
		namespace Private 
		{
			class FSession;
			class FDevice;
		} // Private

		class FModelInstance : public UE::NNE::IModelInstanceCPU
		{
		private:
			FModelInstance(TSharedRef<UE::NNERuntimeIREE::CPU::Private::FSession> InSession);

		public:
			static TSharedPtr<FModelInstance> Make(TSharedRef<UE::NNERuntimeIREE::CPU::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule);

		public:
			using ESetInputTensorShapesStatus = UE::NNE::IModelInstanceCPU::ESetInputTensorShapesStatus;
			using ERunSyncStatus = UE::NNE::IModelInstanceCPU::ERunSyncStatus;

			//~ Begin IModelInstanceCPU Interface
			virtual TConstArrayView<UE::NNE::FTensorDesc> GetInputTensorDescs() const override;
			virtual TConstArrayView<UE::NNE::FTensorDesc> GetOutputTensorDescs() const override;
			virtual TConstArrayView<UE::NNE::FTensorShape> GetInputTensorShapes() const override;
			virtual TConstArrayView<UE::NNE::FTensorShape> GetOutputTensorShapes() const override;
			virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<UE::NNE::FTensorShape> InInputShapes) override;
			virtual ERunSyncStatus RunSync(TConstArrayView<UE::NNE::FTensorBindingCPU> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingCPU> InOutputBindings) override;
			//~ End IModelInstanceCPU Interface

		private:
			TSharedRef<UE::NNERuntimeIREE::CPU::Private::FSession> Session;
		};

		class FModel : public UE::NNE::IModelCPU
		{
		private:
			FModel(TSharedRef<UE::NNERuntimeIREE::CPU::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule);

		public:
			static TSharedPtr<FModel> Make(UE::NNERuntimeIREE::Private::FEnvironment& InEnvironment, const FString& InDirPath, const FString& InSharedLibraryFileName, const FString& InVmfbFileName, const FString& InLibraryQueryFunctionName, const FModuleMetaData& InModuleMetaData);

		public:
			//~ Begin IModelCPU Interface
			virtual TSharedPtr<UE::NNE::IModelInstanceCPU> CreateModelInstanceCPU() override;
			//~ End IModelCPU Interface

		private:
			TSharedRef<UE::NNERuntimeIREE::CPU::Private::FDevice> Device;
			TSharedRef<UE::NNERuntimeIREE::Private::FModule> Module;
		};
	} // CPU

#ifdef WITH_NNE_RUNTIME_IREE_RDG
	namespace RDG
	{
		namespace Private 
		{
			class FSession;
			class FDevice;
		} // Private

		class FModelInstance : public UE::NNE::IModelInstanceRDG
		{
		private:
			FModelInstance(TSharedRef<UE::NNERuntimeIREE::RDG::Private::FSession> InSession);

		public:
			static TSharedPtr<FModelInstance> Make(TSharedRef<UE::NNERuntimeIREE::RDG::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule);

		public:
			using ESetInputTensorShapesStatus = UE::NNE::IModelInstanceRDG::ESetInputTensorShapesStatus;
			using EEnqueueRDGStatus = UE::NNE::IModelInstanceRDG::EEnqueueRDGStatus;

			//~ Begin IModelInstanceRDG Interface
			virtual TConstArrayView<UE::NNE::FTensorDesc> GetInputTensorDescs() const override;
			virtual TConstArrayView<UE::NNE::FTensorDesc> GetOutputTensorDescs() const override;
			virtual TConstArrayView<UE::NNE::FTensorShape> GetInputTensorShapes() const override;
			virtual TConstArrayView<UE::NNE::FTensorShape> GetOutputTensorShapes() const override;
			virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<UE::NNE::FTensorShape> InInputShapes) override;
			virtual EEnqueueRDGStatus EnqueueRDG(FRDGBuilder& RDGBuilder, TConstArrayView<UE::NNE::FTensorBindingRDG> InInputBindings, TConstArrayView<UE::NNE::FTensorBindingRDG> InOutputBindings) override;
			//~ End IModelInstanceRDG Interface

		private:
			TSharedRef<UE::NNERuntimeIREE::RDG::Private::FSession> Session;
		};

		class FModel : public UE::NNE::IModelRDG
		{
		private:
			FModel(TSharedRef<UE::NNERuntimeIREE::RDG::Private::FDevice> InDevice, TSharedRef<UE::NNERuntimeIREE::Private::FModule> InModule);

		public:
			static TSharedPtr<FModel> Make(const FString& InDirPath, TConstArrayView64<uint8> VmfbData, const FModuleMetaData& InModuleMetaData, const TMap<FString, TConstArrayView<uint8>>& Executables);

		public:
			//~ Begin IModelRDG Interface
			virtual TSharedPtr<UE::NNE::IModelInstanceRDG> CreateModelInstanceRDG() override;
			//~ End IModelRDG Interface

		private:
			TSharedRef<UE::NNERuntimeIREE::RDG::Private::FDevice> Device;
			TSharedRef<UE::NNERuntimeIREE::Private::FModule> Module;
		};
	} // namespace RDG
#endif // WITH_NNE_RUNTIME_IREE_RDG
} // UE::NNERuntimeIREE

#endif // WITH_NNE_RUNTIME_IREE