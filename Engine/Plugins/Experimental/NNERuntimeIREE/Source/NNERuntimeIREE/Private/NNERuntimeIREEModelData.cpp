// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEModelData.h"

#include "Containers/Array.h"
#include "NNE.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace UE::NNERuntimeIREE
{
	namespace CPU::Private
	{
		void FModelData::Serialize(FArchive& Ar)
		{
			Ar << GUID;
			Ar << Version;
			Ar << FileId;
			ModuleMetaData.Serialize(Ar);
			FNNERuntimeIREECompilerResultCPU::StaticStruct()->SerializeBin(Ar, &CompilerResult);
			Ar << CompilerHash;
		}

		bool FModelData::Load(TConstArrayView64<uint8> Data)
		{
			FMemoryReaderView Reader(Data, /*bIsPersitent =*/ true);
			Serialize(Reader);
			return !Reader.IsError();
		}

		bool FModelData::Store(TArray64<uint8>& Data)
		{
			FMemoryWriter64 Writer(Data, /*bIsPersitent =*/ true);
			Serialize(Writer);
			return !Writer.IsError();
		}

		bool FModelData::IsSameGuidAndVersion(TConstArrayView64<uint8> Data, FGuid Guid, int32 Version)
		{
			FModelData ModelData{};
			FMemoryReaderView Reader(Data, /*bIsPersitent =*/ true);
			Reader << ModelData.GUID;
			Reader << ModelData.Version;
			return !Reader.IsError() && Guid == ModelData.GUID && Version == ModelData.Version;
		}
	} // CPU::Private

	namespace RDG::Private
	{
		void FModelDataHeader::Serialize(FArchive& Ar)
		{
			Ar << GUID;
			Ar << Version;
			Ar << FileId;
			Ar << ShaderPlatforms;
			Ar << CompilerHash;
		}

		void FModelData::Serialize(FArchive& Ar)
		{
			Header.Serialize(Ar);
			ModuleMetaData.Serialize(Ar);
			FIREECompilerRDGResult::StaticStruct()->SerializeBin(Ar, &CompilerResult);
		}

		bool FModelData::Load(TConstArrayView64<uint8> Data)
		{
			FMemoryReaderView Reader(Data, /*bIsPersitent =*/ true);
			Serialize(Reader);
			return !Reader.IsError();
		}

		bool FModelData::Store(TArray64<uint8>& Data)
		{
			FMemoryWriter64 Writer(Data, /*bIsPersitent =*/ true);
			Serialize(Writer);
			return !Writer.IsError();
		}

		bool FModelData::ReadHeaderAndIsSameGuidAndVersion(FArchive& Ar, FGuid Guid, int32 Version, FModelDataHeader& OutHeader)
		{
			OutHeader.Serialize(Ar);
			return !Ar.IsError() && Guid == OutHeader.GUID && Version == OutHeader.Version;
		}
	} // RDG::Private
} // UE::NNERuntimeIREE
