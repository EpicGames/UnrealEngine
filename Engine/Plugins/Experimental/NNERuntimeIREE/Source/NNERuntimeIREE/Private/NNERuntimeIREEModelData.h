// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "IREECompilerRDG.h"
#include "Misc/SecureHash.h"
#include "NNERuntimeIREECompiler.h"
#include "NNERuntimeIREEMetaData.h"
#include "NNETypes.h"

namespace UE::NNERuntimeIREE::CPU::Private
{
	/**
	 * IREECpu model data class.
	 */
	class FModelData
	{
	public:
		/**
		 * Loads this model from Data that was produced using the Store method
		 *
		 * @return true if loading was successfull, false otherwise, this leaves the object in an undefined state
		 */
		bool Load(TConstArrayView64<uint8> Data);

		/**
		* Serializes this model into Data
		*
		* @return true if storing was successfull, false otherwise
		*/
		bool Store(TArray64<uint8>& Data);

		/**
		 * Check data for matching Guid and Version without deserializing everything.
		 */
		static bool IsSameGuidAndVersion(TConstArrayView64<uint8> Data, FGuid Guid, int32 Version);

		/**
		 * A Guid that uniquely identifies this IREE model data.
		 */
		FGuid GUID;

		/**
		 * Current version of this IREE model data.
		 */
		int32 Version;

		/**
		 * A Guid that uniquely identifies the model.
		 */
		FGuid FileId;

		/**
		 * Module meta data.
		 */
		FModuleMetaData ModuleMetaData;

		/**
		 * Compiler output.
		 */
		FNNERuntimeIREECompilerResultCPU CompilerResult;

		/**
		* Hash of the compiler that created CompilerResult
		*/
		FMD5Hash CompilerHash;

	private:
		void Serialize(FArchive& Ar);
	};
} // UE::NNERuntimeIREE::CPU::Private


namespace UE::NNERuntimeIREE::RDG::Private
{
	/**
	 * IREERdg model data header struct.
	 */
	struct FModelDataHeader
	{
		void Serialize(FArchive& Ar);

		/**
		 * A Guid that uniquely identifies this IREE model data.
		 */
		FGuid GUID;

		/**
		 * Current version of this IREE model data.
		 */
		int32 Version = 0;

		/**
		 * A Guid that uniquely identifies the model.
		 */
		FGuid FileId;

		/**
		 * List of shader platforms supported by this IREE model data.
		 */
		TArray<FString> ShaderPlatforms;

		/**
		* Hash of the compiler that created CompilerResult
		*/
		FMD5Hash CompilerHash;
	};

	/**
	 * IREERdg model data class.
	 */
	class FModelData
	{
	public:
		/**
		 * Loads this model from Data that was produced using the Store method
		 *
		 * @return true if loading was successfull, false otherwise, this leaves the object in an undefined state
		 */
		bool Load(TConstArrayView64<uint8> Data);

		/**
		* Serializes this model into Data
		*
		* @return true if storing was successfull, false otherwise
		*/
		bool Store(TArray64<uint8>& Data);

		/**
		 * Gets Header without loading entire ModelData and checks for matching Guid and Version
		 */
		static bool ReadHeaderAndIsSameGuidAndVersion(FArchive& Ar, FGuid Guid, int32 Version, FModelDataHeader& OutHeader);

		/**
		 * Model data header.
		 */
		FModelDataHeader Header;

		/**
		 * Module meta data.
		 */
		FModuleMetaData ModuleMetaData;

		/**
		 * Compiler output.
		 */
		FIREECompilerRDGResult CompilerResult;

	private:
		void Serialize(FArchive& Ar);
	};
} // UE::NNERuntimeIREE::RDG::Private
