// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEModule.h"

#include "Modules/ModuleManager.h"

#ifdef WITH_NNE_RUNTIME_IREE

#include "CoreMinimal.h"
#include "NNE.h"
#include "NNERuntimeIREEEnvironment.h"
#include "NNERuntimeIREESettings.h"

#if WITH_EDITOR
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "NNEMlirTools_cxx_api.h"
#endif // WITH_EDITOR

#endif // WITH_NNE_RUNTIME_IREE

#include "NNERuntimeIREELog.h"

DEFINE_LOG_CATEGORY(LogNNERuntimeIREE);

namespace UE::NNERuntimeIREE::Private
{
#if WITH_EDITOR
	namespace DllHelper
	{
		void* GetDllHandle(const FString& Path)
		{
			void* Result = nullptr;

			if (!FPaths::FileExists(Path))
			{
				UE_LOGF(LogNNERuntimeIREE, Error, "Failed to find the third party library %ls.", *Path);
				return nullptr;
			}
			
			Result = FPlatformProcess::GetDllHandle(*Path);
			if (!Result)
			{
				UE_LOGF(LogNNERuntimeIREE, Error, "Failed to load the third party library %ls.", *Path);
				return nullptr;
			}

			return Result;
		}
	} // namespace DllHelper
#endif // WITH_EDITOR

#ifdef WITH_NNE_RUNTIME_IREE
	namespace EnvironmentHelper
	{
		void CreateEnvironmentFromSettings(const UNNERuntimeIREESettings* Settings, FEnvironment& Environment)
		{
#if WITH_EDITOR
			const FNNERuntimeIREEThreadingOptions& ThreadingOptions = Settings->EditThreadingOptions;
#else
			const FNNERuntimeIREEThreadingOptions& ThreadingOptions = Settings->GameThreadingOptions;
#endif

			UE::NNERuntimeIREE::Private::FEnvironment::FConfig Config{};
			Config.ThreadingOptions = ThreadingOptions;

			Environment.Configure(Config);
		}
	} // EnvironmentHelper
#endif // WITH_NNE_RUNTIME_IREE
} // namespace UE::NNERuntimeIREE::Private

void FNNERuntimeIREEModule::StartupModule()
{
	using namespace UE::NNERuntimeIREE::Private;

#if WITH_EDITOR
	const FString PluginDir = IPluginManager::Get().FindPlugin("NNERuntimeIREE")->GetBaseDir();
	const FString MlirToolsSharedLibPath = FPaths::Combine(PluginDir, TEXT(UE_STRINGIZE(NNEMLIRTOOLS_SHAREDLIB_PATH)));

	DllHandle = DllHelper::GetDllHandle(MlirToolsSharedLibPath);
	if (!DllHandle)
	{
		UE_LOGF(LogNNERuntimeIREE, Error, "Failed to load MLIR Tools shared library.");
		return;
	}

	auto* GetInterface = reinterpret_cast<const NNEMlirApi* (NNEMLIR_CALL*)(std::uint32_t)>(FPlatformProcess::GetDllExport(DllHandle, TEXT("NNEMlirGetInterface")));
	if (!GetInterface)
	{
		UE_LOGF(LogNNERuntimeIREE, Error, "MLIR Tools shared library symbol 'NNEMlirGetInterface' not found.");
		return;
	}

	const NNEMlirApi* ApiPtr = GetInterface(NNEMLIR_ABI_VERSION);
	if (!ApiPtr)
	{
		UE_LOGF(LogNNERuntimeIREE, Error, "MLIR Tools shared library ABI mismatch (want %d).", NNEMLIR_ABI_VERSION);
		return;
	}

	NNEMlirTools::Api::Initialize(ApiPtr);
#endif // WITH_EDITOR

#ifdef WITH_NNE_RUNTIME_IREE
	Environment = MakeShared<FEnvironment>();

	EnvironmentHelper::CreateEnvironmentFromSettings(GetDefault<UNNERuntimeIREESettings>(), *Environment);

	NNERuntimeIREECpu = NewObject<UNNERuntimeIREECpu>();
	if (NNERuntimeIREECpu.IsValid())
	{
		NNERuntimeIREECpu->Init(Environment.ToSharedRef());

		NNERuntimeIREECpu->AddToRoot();
		UE::NNE::RegisterRuntime(NNERuntimeIREECpu.Get());
	}

	NNERuntimeIREECuda = NewObject<UNNERuntimeIREECuda>();
	if (NNERuntimeIREECuda.IsValid())
	{
		if (NNERuntimeIREECuda->IsAvailable())
		{
			NNERuntimeIREECuda->AddToRoot();
			UE::NNE::RegisterRuntime(NNERuntimeIREECuda.Get());
		}
		else
		{
			NNERuntimeIREECuda.Reset();
		}
	}

	NNERuntimeIREEVulkan = NewObject<UNNERuntimeIREEVulkan>();
	if (NNERuntimeIREEVulkan.IsValid())
	{
		if (NNERuntimeIREEVulkan->IsAvailable())
		{
			NNERuntimeIREEVulkan->AddToRoot();
			UE::NNE::RegisterRuntime(NNERuntimeIREEVulkan.Get());
		}
		else
		{
			NNERuntimeIREEVulkan.Reset();
		}
	}

#ifdef WITH_NNE_RUNTIME_IREE_RDG
	NNERuntimeIREERdg = NewObject<UNNERuntimeIREERdg>();
	if (NNERuntimeIREERdg.IsValid())
	{
		if (NNERuntimeIREERdg->IsAvailable())
		{
			NNERuntimeIREERdg->AddToRoot();
			UE::NNE::RegisterRuntime(NNERuntimeIREERdg.Get());
		}
		else
		{
			NNERuntimeIREERdg.Reset();
		}
	}
#endif // WITH_NNE_RUNTIME_IREE_RDG

#endif // WITH_NNE_RUNTIME_IREE
}

void FNNERuntimeIREEModule::ShutdownModule()
{
#ifdef WITH_NNE_RUNTIME_IREE
	if (NNERuntimeIREECpu.IsValid())
	{
		UE::NNE::UnregisterRuntime(NNERuntimeIREECpu.Get());
		NNERuntimeIREECpu->RemoveFromRoot();
		NNERuntimeIREECpu.Reset();
	}

	if (NNERuntimeIREECuda.IsValid())
	{
		UE::NNE::UnregisterRuntime(NNERuntimeIREECuda.Get());
		NNERuntimeIREECuda->RemoveFromRoot();
		NNERuntimeIREECuda.Reset();
	}

	if (NNERuntimeIREEVulkan.IsValid())
	{
		UE::NNE::UnregisterRuntime(NNERuntimeIREEVulkan.Get());
		NNERuntimeIREEVulkan->RemoveFromRoot();
		NNERuntimeIREEVulkan.Reset();
	}

	if (NNERuntimeIREERdg.IsValid())
	{
		UE::NNE::UnregisterRuntime(NNERuntimeIREERdg.Get());
		NNERuntimeIREERdg->RemoveFromRoot();
		NNERuntimeIREERdg.Reset();
	}

	Environment.Reset();
#endif // WITH_NNE_RUNTIME_IREE
}
	
IMPLEMENT_MODULE(FNNERuntimeIREEModule, NNERuntimeIREE)