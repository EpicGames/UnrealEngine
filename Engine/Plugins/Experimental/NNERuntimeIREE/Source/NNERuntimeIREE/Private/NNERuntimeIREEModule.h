// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#ifdef WITH_NNE_RUNTIME_IREE
#include "NNERuntimeIREE.h"
#include "UObject/WeakObjectPtrTemplates.h"

#endif // WITH_NNE_RUNTIME_IREE

class FNNERuntimeIREEModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void* DllHandle{};

#ifdef WITH_NNE_RUNTIME_IREE
	TWeakObjectPtr<UNNERuntimeIREECpu> NNERuntimeIREECpu;
	TWeakObjectPtr<UNNERuntimeIREECuda> NNERuntimeIREECuda;
	TWeakObjectPtr<UNNERuntimeIREEVulkan> NNERuntimeIREEVulkan;
	TWeakObjectPtr<UNNERuntimeIREERdg> NNERuntimeIREERdg;
	TSharedPtr<UE::NNERuntimeIREE::Private::FEnvironment> Environment;
#endif // WITH_NNE_RUNTIME_IREE
};
