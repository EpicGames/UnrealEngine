// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREESettings.h"

#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"

UNNERuntimeIREESettings::UNNERuntimeIREESettings(const FObjectInitializer& ObjectInitlaizer)
	: Super(ObjectInitlaizer)
{

}

FName UNNERuntimeIREESettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UNNERuntimeIREESettings::GetSectionText() const
{
	return NSLOCTEXT("NNERuntimeIREEPlugin", "NNERuntimeIREESettingsSection", "NNERuntimeIREE");
}
#endif