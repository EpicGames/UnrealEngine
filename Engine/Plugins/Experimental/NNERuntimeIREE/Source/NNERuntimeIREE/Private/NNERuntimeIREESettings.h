// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/ObjectMacros.h"

#include "NNERuntimeIREESettings.generated.h"

/** Specifies the thread affinity group type */
UENUM()
enum class ENNERuntimeIREEThreadingAffinityGroupSpecifierType : uint8
{
	Index 		UMETA(DisplayName = "Specify group by index"),
	Current		UMETA(DisplayName = "Use same group as calling thread"),
	All			UMETA(DisplayName = "Use all groups"),
	Any			UMETA(DisplayName = "Use any group")
};

/** Specifies the processor affinity for a particular thread given by processor group the thread should be assigned to (aka NUMA node, cluster etc., depending on the platform) and the processor it should be scheduled on */
USTRUCT()
struct FNNERuntimeIREEThreadingAffinity
{
	GENERATED_BODY()

	/** How is the group specified? */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Group specifier",
		ToolTip = "Specify the group (type) the thread should be assigned to."))
	ENNERuntimeIREEThreadingAffinityGroupSpecifierType GroupSpecifierType = ENNERuntimeIREEThreadingAffinityGroupSpecifierType::Any;

	/** Group index, only used if node specifier type is 'Index' */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Group index",
		ToolTip = "Specify with an index the group the thread should be assigned to. Only used if group specifier type is 'Index'."))
	int32 GroupIndex = -1;

	/** Logical core index */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Logical core index",
		ToolTip = "Specify with an index the logical core the thread should be scheduled on. Set to -1 to let the thread be scheduled on any core."))
	int32 CoreIndex = -1;
};

/** Specifies the task topology used for multi-threading */
USTRUCT()
struct FNNERuntimeIREETaskTopology
{
	GENERATED_BODY()

	/** Task topology groups */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Task topology groups",
		ToolTip = "Specify the task system topology with zero or more groups. If empty, the runtime will try to use all physical cores."))
	TArray<FNNERuntimeIREEThreadingAffinity> TaskTopologyGroups;
};

/** Threading options to configure single/multi-threading, including task topology for multi-threading case */
USTRUCT()
struct FNNERuntimeIREEThreadingOptions
{
	GENERATED_BODY()

	/** Run single threaded? */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Is single threaded",
		ToolTip = "Should the runtime run single threaded (might result in better performance with very small models)."))
	bool bIsSingleThreaded = true;

	/** Task topology used for multi-threading */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Task topology",
		ToolTip = "For multi-threading you can specify the task system topology."))
	FNNERuntimeIREETaskTopology TaskTopology;
};

/** Settings used to configure NNERuntimeIREE */
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "NNERuntimeIREE"))
class UNNERuntimeIREESettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UNNERuntimeIREESettings(const FObjectInitializer& ObjectInitlaizer);

public:

	/** Threading options in Editor targets */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (DisplayName = "Editor threading options", ToolTip = "Threading options in Editor targets", ConfigRestartRequired = true))
	FNNERuntimeIREEThreadingOptions EditThreadingOptions{};

	/** Threading options in Non-Editor (Game, Program, ...) targets */
	UPROPERTY(Config, EditAnywhere, Category = "IREE Runtime", meta = (DisplayName = "Game threading options", ToolTip = "Threading options in Non-Editor (Game, Program, ...) targets", ConfigRestartRequired = true))
	FNNERuntimeIREEThreadingOptions GameThreadingOptions{};

	// Begin UDeveloperSettings Interface
	NNERUNTIMEIREE_API virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	NNERUNTIMEIREE_API virtual FText GetSectionText() const override;
	// END UDeveloperSettings Interface
#endif

};

/** NNERuntimeIREECpu compiler flag settings */
USTRUCT()
struct FNNERuntimeIREECpuCompilerFlags
{
	GENERATED_BODY()

	/** The Platform the flags should be applied to, or leave empty to apply to all */
	UPROPERTY(EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Platform",
		ToolTip = "The Platform the flags should be applied to, or leave empty to apply to all."))
	FString Platform;

	/** The name of the target these flags should be applied to, or leave empty to apply to all */
	UPROPERTY(EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Target",
		ToolTip = "The name of the target these flags should be applied to, or leave empty to apply to all."))
	FString Target;

	/** The flags that get appended to the iree compiler arguments on the specified platform and target */
	UPROPERTY(EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Flags",
		ToolTip = "The flags that get appended to the iree compiler arguments on the specified platform and target."))
	FString Flags;
};

/** NNERuntimeIREECpu compiler settings */
USTRUCT()
struct FNNERuntimeIREECpuCompilerSettings
{
	GENERATED_BODY()

	/** NNERuntimeIREECpu compiler flag settings */
	UPROPERTY(EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Compiler Flags", 
		ToolTip = "List of compiler flags. The flags of the first item that has the correct platform and target get appended to the existing iree compiler arguments.",
		TitleProperty="{Platform}, {Target}: {Flags}"))
	TArray<FNNERuntimeIREECpuCompilerFlags> CompilerFlags;
};

/** RuntimeSettings for NNERuntimeIREECpu */
UCLASS()
class UNNERuntimeIREECpuSettings : public UObject
{
	GENERATED_BODY()

public:

	/** NNERuntimeIREECpu compiler settings */
	UPROPERTY(EditAnywhere, Category = "IREE Runtime", meta = (
		DisplayName = "Compiler settings", 
		ToolTip = "Settings to configure compilation.",
		NoResetToDefault))
	FNNERuntimeIREECpuCompilerSettings CompilerSettings;
};
