// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeIREEEditor : ModuleRules
{
	public NNERuntimeIREEEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"MainFrame",
				"NNE",
				"NNERuntimeIREE",
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);
	}
}
