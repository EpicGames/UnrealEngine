// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEEditorModule.h"
#include "NNERuntimeIREELog.h"

#define LOCTEXT_NAMESPACE "FNNERuntimeIREEEditorModule"

void FNNERuntimeIREEEditorModule::StartupModule()
{
}

void FNNERuntimeIREEEditorModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNNERuntimeIREEEditorModule, NNERuntimeIREEEditor)