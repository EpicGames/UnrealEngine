// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEModelDataFactory.h"

#include "CoreMinimal.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Reply.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeIREELog.h"
#include "NNERuntimeIREEMetaData.h"
#include "Serialization/MemoryWriter.h"
#include "Subsystems/ImportSubsystem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class SMLIRImportWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMLIRImportWindow) {}
		SLATE_ARGUMENT(TConstArrayView<UE::NNERuntimeIREE::FFunctionMetaData>, MetaData)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TSharedRef<SWidget> OnSelectFunctionContent();
	void OnSelectFunctionChange(int32 Index);
	FText GetCurrentSelectedFunction() const;
	int32 GetCurrentSelectedFunctionIndex() const;

	FReply OnImportButtonClicked();
	bool ImportButtonClicked() const;

private:
	bool bImportButtonClicked = false;
	int32 Selection = 0;
	TConstArrayView<UE::NNERuntimeIREE::FFunctionMetaData> MetaData;
	TWeakPtr<SWindow> WidgetWindow;
};

void SMLIRImportWindow::Construct(const FArguments& InArgs)
{
	bImportButtonClicked = false;
	Selection = 0;
	MetaData = InArgs._MetaData;
	WidgetWindow = InArgs._WidgetWindow;

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(10.0f, 10.0f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("UnrealEd", "NNERuntimeIREEImportFunctionSelectionText", "Neural network main function:"))
			]
			+SVerticalBox::Slot()
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &SMLIRImportWindow::OnSelectFunctionContent)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SMLIRImportWindow::GetCurrentSelectedFunction)
				]
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 10.0f))
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(NSLOCTEXT("UnrealEd", "NNERuntimeIREEImportButton", "Import"))
				.OnClicked(this, &SMLIRImportWindow::OnImportButtonClicked)
			]
		]
		
	];
}

TSharedRef<SWidget> SMLIRImportWindow::OnSelectFunctionContent()
{
	FMenuBuilder MenuBuilder(true, NULL);
	for (int32 i = 0; i < MetaData.Num(); i++)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(this, &SMLIRImportWindow::OnSelectFunctionChange, i));
		MenuBuilder.AddMenuEntry(FText::FromName(FName(MetaData[i].Name)), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}
	return MenuBuilder.MakeWidget();
}

void SMLIRImportWindow::OnSelectFunctionChange(int32 Index)
{
	Selection = Index;
}

FText SMLIRImportWindow::GetCurrentSelectedFunction() const
{
	if (Selection >= 0 && Selection < MetaData.Num())
	{
		return FText::FromString(MetaData[Selection].Name);
	}
	return FText();
}

int32 SMLIRImportWindow::GetCurrentSelectedFunctionIndex() const
{
	return Selection;
}

FReply SMLIRImportWindow::OnImportButtonClicked()
{
	bImportButtonClicked = true;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

bool SMLIRImportWindow::ImportButtonClicked() const
{
	return bImportButtonClicked;
}

UNNERuntimeIREEModelDataFactory::UNNERuntimeIREEModelDataFactory(const FObjectInitializer& ObjectInitializer) : UFactory(ObjectInitializer)
{
	bCreateNew = false;
	bEditorImport = true;
	SupportedClass = UNNEModelData::StaticClass();
	ImportPriority = DefaultImportPriority;
	Formats.Add("mlir;Multi-Level Intermediate Representation Format");
}

UObject* UNNERuntimeIREEModelDataFactory::FactoryCreateBinary(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR * Type, const uint8 *& Buffer, const uint8 * BufferEnd, FFeedbackContext* Warn)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);

	if (!Type || !Buffer || !BufferEnd || BufferEnd - Buffer <= 0)
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
		return nullptr;
	}

	TConstArrayView<uint8> BufferView = MakeArrayView(Buffer, BufferEnd - Buffer);

	UE::NNERuntimeIREE::FModuleMetaData ModuleMetaData{};
	if (!ModuleMetaData.ParseFromBuffer(BufferView))
	{
		UE_LOGF(LogNNERuntimeIREE, Error, "UNNERuntimeIREEModelDataFactory failed to parse the models meta data");
		return nullptr;
	}
	if (ModuleMetaData.FunctionMetaData.IsEmpty())
	{
		UE_LOGF(LogNNERuntimeIREE, Error, "UNNERuntimeIREEModelDataFactory no public functions found in models meta data");
		return nullptr;
	}

	if (ModuleMetaData.FunctionMetaData.Num() > 1)
	{
		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(NSLOCTEXT("UnrealEd", "NNERuntimeIREEImportWindowTitle", "MLIR Import Options"))
			.SizingRule(ESizingRule::Autosized)
			.AutoCenter(EAutoCenter::PreferredWorkArea);

		TSharedPtr<SMLIRImportWindow> ImportWindow;
		Window->SetContent
		(
			SAssignNew(ImportWindow, SMLIRImportWindow)
			.MetaData(ModuleMetaData.FunctionMetaData)
			.WidgetWindow(Window)
		);

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		if (!ImportWindow->ImportButtonClicked())
		{
			UE_LOGF(LogNNERuntimeIREE, Error, "UNNERuntimeIREEModelDataFactory could not import the model! Please select the neural network main function in the import dialog!");
			return nullptr;
		}

		UE::NNERuntimeIREE::FFunctionMetaData Swapper = ModuleMetaData.FunctionMetaData[0];
		ModuleMetaData.FunctionMetaData[0] = ModuleMetaData.FunctionMetaData[ImportWindow->GetCurrentSelectedFunctionIndex()];
		ModuleMetaData.FunctionMetaData[ImportWindow->GetCurrentSelectedFunctionIndex()] = Swapper;
	}

	TArray<uint8> MetaDataByteArray;
	FMemoryWriter Writer(MetaDataByteArray, /*bIsPersitent =*/ true);
	ModuleMetaData.Serialize(Writer);

	TMap<FString, TConstArrayView64<uint8>> AdditionalFileData;
	AdditionalFileData.Add("IREEModuleMetaData", MetaDataByteArray);

	UNNEModelData* ModelData = NewObject<UNNEModelData>(InParent, Class, Name, Flags);
	ModelData->Init(Type, BufferView, AdditionalFileData);

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ModelData);

	return ModelData;
}

bool UNNERuntimeIREEModelDataFactory::FactoryCanImport(const FString & Filename)
{
	return Filename.EndsWith(FString("mlir"));
}
