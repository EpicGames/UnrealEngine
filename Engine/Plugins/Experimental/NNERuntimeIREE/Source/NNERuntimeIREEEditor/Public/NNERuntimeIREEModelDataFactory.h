// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UObjectGlobals.h"
#include "Factories/Factory.h"

#include "NNERuntimeIREEModelDataFactory.generated.h"

UCLASS()
class NNERUNTIMEIREEEDITOR_API UNNERuntimeIREEModelDataFactory : public UFactory
{
	GENERATED_BODY()

public:
	UNNERuntimeIREEModelDataFactory(const FObjectInitializer& ObjectInitializer);

public:
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateBinary(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface
};