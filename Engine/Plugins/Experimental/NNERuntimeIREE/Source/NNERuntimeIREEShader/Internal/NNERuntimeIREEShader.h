// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "Shader.h"
#include "ShaderParameters.h"
#include "NNERuntimeIREEShaderType.h"
#include "NNERuntimeIREEShaderShared.h"

class NNERUNTIMEIREESHADER_API FNNERuntimeIREEShader : public FShader
{
	DECLARE_SHADER_TYPE(FNNERuntimeIREEShader, NNERuntimeIREE);

public:
	FNNERuntimeIREEShader() = default;
	FNNERuntimeIREEShader(const FNNERuntimeIREEShaderType::CompiledShaderInitializerType & Initializer);

	MS_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT) struct FParameters
	{
	};
};

#endif // WITH_NNE_RUNTIME_IREE_SHADER