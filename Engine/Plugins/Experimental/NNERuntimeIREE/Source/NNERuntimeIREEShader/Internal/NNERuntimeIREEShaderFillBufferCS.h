// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNERuntimeIREEShader::Internal
{

class FFillBufferConstants
{
public:
	static const uint32 THREAD_GROUP_SIZE{ 256 };
};

class NNERUNTIMEIREESHADER_API FFillBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFillBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FFillBufferCS, FGlobalShader)

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, TargetBuffer)
		SHADER_PARAMETER(FUintVector4, Fill)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
};

} // UE::NNERuntimeIREEShader::Internal

#endif // WITH_NNE_RUNTIME_IREE_SHADER