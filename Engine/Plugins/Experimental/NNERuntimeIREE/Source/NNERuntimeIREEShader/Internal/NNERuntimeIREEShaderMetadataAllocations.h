// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "ShaderParameterMetadata.h"
#include "Templates/UniquePtr.h"

struct NNERUNTIMEIREESHADER_API FNNERuntimeIREEShaderParametersMetadataAllocations
{
	/** Allocated metadata. Should include the parent metadata allocation. */
	TUniquePtr<FShaderParametersMetadata> ShaderParameterMetadatas;
	/** Allocated name dictionary. */
	TArray<FString> Names;

	FNNERuntimeIREEShaderParametersMetadataAllocations() = default;
	FNNERuntimeIREEShaderParametersMetadataAllocations(FNNERuntimeIREEShaderParametersMetadataAllocations& Other) = delete;

	~FNNERuntimeIREEShaderParametersMetadataAllocations();
};

#endif // WITH_NNE_RUNTIME_IREE_SHADER