// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "CoreMinimal.h"
#include "NNERuntimeIREEShaderCompileResult.h"
#include "Engine/EngineTypes.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "RenderDeferredCleanup.h"
#include "RHI.h"
#include "SceneTypes.h"
#include "Shader.h"
#include "ShaderCompiler.h"
#include "Templates/RefCounting.h"

struct FNNERuntimeIREEShaderParametersMetadataAllocations;
class FNNERuntimeIREEResource;
class FNNERuntimeIREEShader;

/** Stores outputs from the shader compile that need to be saved. */
class FNNERuntimeIREECompilationOutput
{
	DECLARE_TYPE_LAYOUT(FNNERuntimeIREECompilationOutput, NonVirtual);
public:
	FNNERuntimeIREECompilationOutput()
	{}
};

/** Contains all the information needed to uniquely identify a FNNERuntimeIREEShaderMapID. */
class FNNERuntimeIREEShaderMapId
{
	DECLARE_TYPE_LAYOUT(FNNERuntimeIREEShaderMapId, NonVirtual);
public:
	/** Feature level that the shader map is going to be compiled for.  */
	LAYOUT_FIELD(ERHIFeatureLevel::Type, FeatureLevel);

	/**
	 * The hash computed from the shader code generated by the library.
	 */
	LAYOUT_FIELD(uint64, ShaderCodeHash);

	/** Shader types of shaders that are inlined in this shader map in the DDC. */
	LAYOUT_FIELD(TMemoryImageArray<FShaderTypeDependency>, ShaderTypeDependencies);

	/*
	* Type layout parameters of the memory image
	*/
	LAYOUT_FIELD(FPlatformTypeLayoutParameters, LayoutParams);

public:
	FNNERuntimeIREEShaderMapId()
		: FeatureLevel(ERHIFeatureLevel::SM5)
	{ }

	~FNNERuntimeIREEShaderMapId() = default;

#if WITH_EDITOR
	void SetShaderDependencies(const TArray<FShaderType*>& InShaderTypes, EShaderPlatform InShaderPlatform);
#endif

	//void Serialize(FArchive& Ar);

	friend uint32 GetTypeHash(const FNNERuntimeIREEShaderMapId& Ref)
	{
		return GetTypeHash(Ref.ShaderCodeHash);
	}

	SIZE_T GetSizeBytes() const
	{
		return sizeof(*this);
	}

	/** Hashes the kernel specific part of this shader map Id. */
	uint64 GetKernelHash() const;

	/**
	 * Tests this set against another for equality, disregarding override settings.
	 *
	 * @param InReferenceSet	The set to compare against
	 * @return					true if the sets are equal
	 */
	bool operator==(const FNNERuntimeIREEShaderMapId& InReferenceSet) const;

	bool operator!=(const FNNERuntimeIREEShaderMapId& InReferenceSet) const
	{
		return !(*this == InReferenceSet);
	}

#if WITH_EDITOR
	/** Appends string representations of this Id to a key string. */
	void AppendKeyString(FString& OutKeyString) const;
#endif // WITH_EDITOR

	/** Returns true if the requested shader type is a dependency of this shader map Id. */
	bool ContainsShaderType(const FShaderType* ShaderType) const;
};

class FNNERuntimeIREEShaderMapContent : public FShaderMapContent
{
	using Super = FShaderMapContent;
	friend class FNNERuntimeIREEShaderMap;
	DECLARE_TYPE_LAYOUT(FNNERuntimeIREEShaderMapContent, NonVirtual);
private:
	explicit FNNERuntimeIREEShaderMapContent(EShaderPlatform InPlatform) : Super(InPlatform) {}

	/** The kernel's user friendly name */
	LAYOUT_FIELD(FMemoryImageString, FriendlyName);

	/** The static parameter set that this shader map was compiled with */
	LAYOUT_FIELD(FNNERuntimeIREEShaderMapId, ShaderMapId);

	/** Shader compilation output */
	LAYOUT_FIELD(FNNERuntimeIREECompilationOutput, CompilationOutput);
};


/**
 * All the shader permutations for a kernel.
 */
class FNNERuntimeIREEShaderMap : public TShaderMap<FNNERuntimeIREEShaderMapContent, FShaderMapPointerTable>, public FDeferredCleanupInterface
{
public:
	using Super = TShaderMap<FNNERuntimeIREEShaderMapContent, FShaderMapPointerTable>;

	/**
	 * Finds the shader map for a kernel.
	 * @param InShaderMapId - Id to find
	 * @param Platform - The platform to lookup for
	 * @return nullptr if no cached shader map was found.
	 */
	static FNNERuntimeIREEShaderMap* FindId(const FNNERuntimeIREEShaderMapId& InShaderMapId, EShaderPlatform InPlatform);

	// ShaderMap interface
	template<typename ShaderType> TShaderRef<ShaderType> GetShader(int32 PermutationId) const { return TShaderRef<ShaderType>(GetContent()->GetShader<ShaderType>(PermutationId), *this); }
	TShaderRef<FShader> GetShader(FShaderType* ShaderType, int32 PermutationId) const { return TShaderRef<FShader>(GetContent()->GetShader(ShaderType, PermutationId), *this); }

	FNNERuntimeIREEShaderMap();
	~FNNERuntimeIREEShaderMap();

#if WITH_EDITOR
	/**
	 * Compiles the shaders for a kernel and caches them in this shader map.
	 * @param InKernel - The kernel to compile shaders for.
	 * @param InShaderMapId - the set of static parameters to compile for
	 * @param InPlatform - The platform to compile to
	 */
	void Compile(
		FNNERuntimeIREEResource* InKernel,
		const FNNERuntimeIREEShaderMapId& InShaderMapId,
		TRefCountPtr<FSharedShaderCompilerEnvironment> InCompilationEnvironment,
		const FNNERuntimeIREECompilationOutput& InCompilationOutput,
		EShaderPlatform InPlatform,
		bool bSynchronousCompile,
		bool bApplyCompletedShaderMapForRendering
		);

	/** Sorts the incoming compiled jobs into the appropriate shader maps, and finalizes this shader map so that it can be used for rendering. */
	bool ProcessCompilationResults(const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutResultIndex, float& InOutTimeBudget);
#endif // WITH_EDITOR

	/**
	 * Checks whether the shader map is missing any shader types necessary for the given kernel.
	 * @param InKernel - The NNERuntimeIREE shader resource which is checked.
	 * @return True if the shader map has all of the shader types necessary.
	 */
	bool IsComplete(const FNNERuntimeIREEResource* InKernel, bool bSilent);

	/** Builds a list of the shaders in a shader map. */
	void GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const;
	virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const override;
	virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const override;

	/** Registers a NNERuntimeIREE shader map in the global map so it can be used. */
	void Register(EShaderPlatform InShaderPlatform);

	// Reference counting.
	void AddRef();
	void Release();

	/** Serializes the shader map. */
	bool Serialize(FArchive& Ar);

	// Accessors
	const FNNERuntimeIREEShaderMapId& GetShaderMapId() const { return GetContent()->ShaderMapId; }
	EShaderPlatform GetShaderPlatform() const { return GetContent()->GetShaderPlatform(); }
	const FMemoryImageString& GetFriendlyName() const { return GetContent()->FriendlyName; }
	bool IsCompilationFinalized() const { return bCompilationFinalized; }
	bool CompiledSuccessfully() const { return bCompiledSuccessfully; }

#if WITH_EDITOR
	static TMap<TRefCountPtr<FNNERuntimeIREEShaderMap>, TArray<FNNERuntimeIREEResource*> > &GetInFlightShaderMaps() { return ShaderMapsBeingCompiled; }

	void SetCompiledSuccessfully(bool bSuccess) { bCompiledSuccessfully = bSuccess; }
#endif // WITH_EDITOR

	int32 GetNumRefs() const { return NumRefs; }
	uint32 GetCompilingId()  { return CompilingId; }

private:
#if WITH_EDITOR
	FShader* ProcessCompilationResultsForSingleJob(FShaderCompileJob& InSingleJob, uint64 InShaderMapHash);
#endif

	bool IsIREEShaderComplete(const FNNERuntimeIREEResource* InKernel, const FNNERuntimeIREEShaderType* InShaderType, bool bSilent);

	/**
	 * A global map from a kernel's ID and static switch set to any shader map cached for that kernel.
	 * Note: this does not necessarily contain all kernel shader maps in memory.  Shader maps with the same key can evict each other.
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TMap<FNNERuntimeIREEShaderMapId, FNNERuntimeIREEShaderMap*> GIdToIREEShaderMap[SP_NumPlatforms];

	/**
	 * All kernel shader maps in memory.
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TArray<FNNERuntimeIREEShaderMap*> AllKernelShaderMaps;

#if WITH_EDITOR
	/** Tracks resources and their shader maps that need to be compiled but whose compilation is being deferred. */
	static TMap<TRefCountPtr<FNNERuntimeIREEShaderMap>, TArray<FNNERuntimeIREEResource*> > ShaderMapsBeingCompiled;
#endif

	/** Uniquely identifies this shader map during compilation, needed for deferred compilation where shaders from multiple shader maps are compiled together. */
	uint32 CompilingId;

	mutable int32 NumRefs;

	/** Used to catch errors where the shader map is deleted directly. */
	bool bDeletedThroughDeferredCleanup;

	/** Indicates whether this shader map has been registered in GIdToIREEShaderMap */
	uint32 bRegistered : 1;

	/**
	 * Indicates whether this shader map has had ProcessCompilationResults called after Compile.
	 * The shader map must not be used on the rendering thread unless bCompilationFinalized is true.
	 */
	uint32 bCompilationFinalized : 1;

	uint32 bCompiledSuccessfully : 1;

};

class NNERUNTIMEIREESHADER_API FNNERuntimeIREEResource
{
public:
	FNNERuntimeIREEResource();
	virtual ~FNNERuntimeIREEResource();

	/**
	 * Caches the shaders for this kernel with no static parameters on the given platform.
	 * This is used by UKernel
	 */
	bool CacheShaders(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous);
	bool CacheShaders(const FNNERuntimeIREEShaderMapId& InShaderMapId, EShaderPlatform InPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous);

	/**
	 * Should the shader for this kernel with the given platform and shader type combination be compiled
	 *
	 * @param InPlatform		The platform currently being compiled for
	 * @param InShaderType		Which shader is being compiled
	 *
	 * @return true if the shader should be compiled
	 */
	virtual bool ShouldCache(EShaderPlatform InPlatform, const FShaderType* InShaderType) const;

	bool SerializeShaderMap(FArchive& Ar);

	void GetDependentShaderTypes(EShaderPlatform InPlatform, TArray<FShaderType*>& OutShaderTypes) const;
	virtual void GetShaderMapId(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, FNNERuntimeIREEShaderMapId& OutId) const;

	/**
	 * Called when compilation finishes, after the GameThreadShaderMap is set and the render command to set the RenderThreadShaderMap is queued
	 */
	virtual void NotifyCompilationFinished(FString const& ResultMessage);

#if WITH_EDITOR
	/**
	 * Cancels all outstanding compilation jobs
	 */
	void CancelCompilation();

	/**
	 * Blocks until compilation has completed. Returns immediately if a compilation is not outstanding.
	 */
	void FinishCompilation();
#endif // WITH_EDITOR

	// Accessors.
	FNNERuntimeIREEShaderCompileResults const& GetCompilationResults() const { return CompilationResults; }
	void SetCompilationResults(FNNERuntimeIREEShaderCompileResults const& InCompilationResults) { CompilationResults = InCompilationResults; }

	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	FNNERuntimeIREEShaderMap* GetGameThreadShaderMap() const
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		return GameThreadShaderMap;
	}

	/** Returns owner name for tracking */
	FName GetOwnerFName() const
	{
		return AssetPath;
	}

	/** Note: SetRenderingThreadShaderMap must also be called with the same value, but from the rendering thread. */
	void SetGameThreadShaderMap(FNNERuntimeIREEShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
	}

	/** Note: SetGameThreadShaderMap must also be called with the same value, but from the game thread. */
	void SetRenderingThreadShaderMap(FNNERuntimeIREEShaderMap* InShaderMap);

	void AddCompileId(uint32 InIdentifier) 
	{
		OutstandingCompileShaderMapIds.Add(InIdentifier);
	}

	void RemoveOutstandingCompileId(const int32 InOldOutstandingCompileShaderMapId);

	const FString& GetHLSLSource() const
	{
		return ShaderSource;
	}

	const FString& GetEntryPoint() const
	{
		return ShaderEntryPoint;
	}

	const FShaderParametersMetadata* GetShaderParamMetadata() const
	{
		return ShaderParameterMetadata;
	}

	const FString& GetFriendlyName() const { return FriendlyName; }

	int32 GetNumPermutations() const { return 1; }

	void SetupResource(
		ERHIFeatureLevel::Type InFeatureLevel,
		FString const& InFriendlyName,
		FString const& InShaderEntryPoint,
		FString const& InShaderHashKey,
		FString const& InShaderSource,
		TUniquePtr<FNNERuntimeIREEShaderParametersMetadataAllocations> InShaderParameterMetadataAllocations,
		FShaderParametersMetadata* InShaderParameterMetadata,
		FName const& InAssetPath,
		TConstArrayView<uint32> InBufferBindings
	);

	TShaderRef<FNNERuntimeIREEShader> GetShader(int32 PermutationId) const;

	bool IsSame(const FNNERuntimeIREEShaderMapId& InId) const;

	uint32 GetBindingIndex(uint32 BufferIdx) const;

protected:
#if WITH_EDITOR
	/**
	 * Fills the passed array with IDs of shader maps unfinished compilation jobs.
	 */
	void GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& OutShaderMapIds);
#endif // WITH_EDITOR

private:
	FNNERuntimeIREEShaderCompileResults CompilationResults;

	/** 
	 * Game thread tracked shader map, which is ref counted and manages shader map lifetime. 
	 * The shader map uses deferred deletion so that the rendering thread has a chance to process a release command when the shader map is no longer referenced.
	 * Code that sets this is responsible for updating RenderingThreadShaderMap in a thread safe way.
	 * During an async compile, this will be NULL and will not contain the actual shader map until compilation is complete.
	 */
	TRefCountPtr<FNNERuntimeIREEShaderMap> GameThreadShaderMap = nullptr;

	/** 
	 * Shader map for this FComputeKernelResource which is accessible by the rendering thread. 
	 * This must be updated along with GameThreadShaderMap, but on the rendering thread.
	 */
	FNNERuntimeIREEShaderMap* RenderingThreadShaderMap = nullptr;

	/** Name of shader main function. */
	FString ShaderEntryPoint;

	/** Cached shader source for compilation. */
	FString ShaderSource;

	/** 
	 * Hash computed from shader code. If multiple kernels give
	 * the same output, same shader resource will be shared 
	 */
	uint64 ShaderCodeHash = 0;

	TUniquePtr<FNNERuntimeIREEShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations;

	/** Shader parameter metadata. It is expected that object is owned by ShaderParameterMetadataAllocations.  */
	FShaderParametersMetadata const* ShaderParameterMetadata;

	/** Binding indices for storage buffers, needed for IREE command buffer dispatches (buffers might alias and two of them map to the same RDG buffer) */
	TArray<uint32> BufferBindings;

#if WITH_EDITOR
	/**
	 * Compiles this kernel for InPlatform, storing the result in OutShaderMap if the compile was synchronous
	 */
	bool BeginCompileShaderMap(
		const FNNERuntimeIREEShaderMapId& InShaderMapId,
		EShaderPlatform InPlatform, 
		TRefCountPtr<FNNERuntimeIREEShaderMap>& OutShaderMap, 
		bool bApplyCompletedShaderMapForRendering,
		bool bSynchronous = false);
#endif // WITH_EDITOR

	FString FriendlyName;

	/** Asset using this resource */
	FName AssetPath;

	/** 
	 * Contains the compiling id of this shader map when it is being compiled asynchronously. 
	 * This can be used to access the shader map during async compiling, since GameThreadShaderMap will not have been set yet.
	 */
	TArray<int32, TInlineAllocator<1> > OutstandingCompileShaderMapIds;

	/** Feature level that this kernel is representing. */
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM5;

	uint32 bLoadedCookedShaderMapId : 1;
	FNNERuntimeIREEShaderMapId CookedShaderMapId;
};

#endif // WITH_NNE_RUNTIME_IREE_SHADER