// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeIREEShader : ModuleRules
{
	public NNERuntimeIREEShader(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Projects",
				"RenderCore",
				"RHI",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_SHADER");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_SHADER");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_IREE_SHADER");
		}
	}
}
