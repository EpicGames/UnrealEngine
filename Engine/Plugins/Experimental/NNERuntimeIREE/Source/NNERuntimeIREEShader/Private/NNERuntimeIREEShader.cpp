// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEShader.h"

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "NNERuntimeIREEShaderType.h"

IMPLEMENT_SHADER_TYPE(, FNNERuntimeIREEShader, TEXT("/Plugin/NNERuntimeIREEShader/NNERuntimeIREEShader.usf"), TEXT("__"), SF_Compute)

FNNERuntimeIREEShader::FNNERuntimeIREEShader(const FNNERuntimeIREEShaderType::CompiledShaderInitializerType& Initializer)
	: FShader(Initializer)
{
	const FShaderParametersMetadata& ShaderParametersMetadata = static_cast<const FNNERuntimeIREEShaderType::FParameters*>(Initializer.Parameters)->ShaderParamMetadata;

	Bindings.BindForLegacyShaderParameters(this, Initializer.PermutationId, Initializer.ParameterMap, ShaderParametersMetadata, true);
}

#endif // WITH_NNE_RUNTIME_IREE_SHADER