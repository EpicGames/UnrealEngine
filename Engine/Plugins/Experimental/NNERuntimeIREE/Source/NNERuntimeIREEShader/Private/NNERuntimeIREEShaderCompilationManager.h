// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "NNERuntimeIREEShaderShared.h"
#include "ShaderCompiler.h"

/** Results for a single compiled shader map. */
struct FNNERuntimeIREEShaderMapCompileResults
{
	FNNERuntimeIREEShaderMapCompileResults() :
		NumJobsQueued(0),
		bAllJobsSucceeded(true),
		bRecreateComponentRenderStateOnCompletion(false)
	{}

	int32 NumJobsQueued;
	bool bAllJobsSucceeded;
	bool bRecreateComponentRenderStateOnCompletion;
	TArray<FShaderCommonCompileJobPtr> FinishedJobs;
};

/** Results for a single compiled and finalized shader map. */
struct FNNERuntimeIREEShaderMapFinalizeResults : public FNNERuntimeIREEShaderMapCompileResults
{
	/** Tracks finalization progress on this shader map. */
	int32 FinalizeJobIndex;

	FNNERuntimeIREEShaderMapFinalizeResults(const FNNERuntimeIREEShaderMapCompileResults& InCompileResults) :
		FNNERuntimeIREEShaderMapCompileResults(InCompileResults),
		FinalizeJobIndex(0)
	{}
};

#if WITH_EDITOR

/** Handles finished shader compile jobs, applying of the shaders to their config asset, and some error handling. */
class FNNERuntimeIREEShaderCompilationManager
{
public:
	void Tick(float DeltaSeconds);
	void AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs);
	void FinishCompilation(const TCHAR* InKernelName, const TArray<int32>& ShaderMapIdsToFinishCompiling);

private:
	void ProcessAsyncResults();
	void ProcessCompiledNNERuntimeIREEShaderMaps(TMap<int32, FNNERuntimeIREEShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget);

	TArray<FShaderCommonCompileJobPtr> JobQueue;

	/** Map from shader map Id to the compile results for that map, used to gather compiled results. */
	TMap<int32, FNNERuntimeIREEShaderMapCompileResults> NNERuntimeIREEShaderMapJobs;

	/** Map from shader map id to results being finalized.  Used to track shader finalizations over multiple frames. */
	TMap<int32, FNNERuntimeIREEShaderMapFinalizeResults> PendingFinalizeNNERuntimeIREEShaderMaps;
};

extern FNNERuntimeIREEShaderCompilationManager GNNERuntimeIREEShaderCompilationManager;

#endif // WITH_EDITOR
#endif // WITH_NNE_RUNTIME_IREE_SHADER