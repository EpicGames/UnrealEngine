// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEShaderFillBufferCS.h"

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

namespace UE::NNERuntimeIREEShader::Internal
{

void FFillBufferCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), FFillBufferConstants::THREAD_GROUP_SIZE);
}

bool FFillBufferCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return true;
}



IMPLEMENT_GLOBAL_SHADER(FFillBufferCS, "/Plugin/NNERuntimeIREEShader/NNERuntimeIREEShaderFillBuffer.usf", "Main", SF_Compute);

} // UE::NNERuntimeIREEShader::Internal

#endif // WITH_NNE_RUNTIME_IREE_SHADER