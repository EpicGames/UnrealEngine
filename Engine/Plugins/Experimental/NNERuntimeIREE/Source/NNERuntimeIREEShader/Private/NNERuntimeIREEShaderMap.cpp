// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEShader.h"

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "NNERuntimeIREEShaderShared.h"
#include "NNERuntimeIREEShaderCompilationManager.h"
#include "NNERuntimeIREEShaderLog.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "RHIShaderFormatDefinitions.inl"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "ShaderSerialization.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectGlobals.h"
#include "DataDrivenShaderPlatformInfo.h"

#if ENABLE_COOK_STATS
namespace NNERuntimeIREEShaderCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static int32 ShadersCompiled = 0;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("NNERuntimeIREEShader.Usage"), TEXT(""));
		AddStat(TEXT("NNERuntimeIREEShader.Misc"), FCookStatsManager::CreateKeyValueArray(
			TEXT("ShadersCompiled"), ShadersCompiled
		));
	});
}
#endif

//
// Globals
//
TMap<FNNERuntimeIREEShaderMapId, FNNERuntimeIREEShaderMap*> FNNERuntimeIREEShaderMap::GIdToIREEShaderMap[SP_NumPlatforms];
TArray<FNNERuntimeIREEShaderMap*> FNNERuntimeIREEShaderMap::AllKernelShaderMaps;

#if WITH_EDITOR
/** 
 * Tracks FNNERuntimeIREEResource and their shader maps that are being compiled.
 * Uses a TRefCountPtr as this will be the only reference to a shader map while it is being compiled.
 */
TMap<TRefCountPtr<FNNERuntimeIREEShaderMap>, TArray<FNNERuntimeIREEResource*> > FNNERuntimeIREEShaderMap::ShaderMapsBeingCompiled;
#endif // WITH_EDITOR

static inline bool ShouldCacheNNERuntimeIREEShader(const FNNERuntimeIREEShaderType* InShaderType, EShaderPlatform InPlatform, const FNNERuntimeIREEResource* InKernel)
{
	return InShaderType->ShouldCache(InPlatform, InKernel) && InKernel->ShouldCache(InPlatform, InShaderType);
}

/** Hashes the kernel specific part of this shader map Id. */
uint64 FNNERuntimeIREEShaderMapId::GetKernelHash() const
{
	FXxHash64Builder HashState;

	HashState.Update((const uint8*)&ShaderCodeHash, sizeof(ShaderCodeHash));
	HashState.Update((const uint8*)&FeatureLevel, sizeof(FeatureLevel));

	return HashState.Finalize().Hash;
}

/** 
* Tests this set against another for equality, disregarding override settings.
* 
* @param ReferenceSet	The set to compare against
* @return				true if the sets are equal
*/
bool FNNERuntimeIREEShaderMapId::operator==(const FNNERuntimeIREEShaderMapId& InReferenceSet) const
{
	if (  ShaderCodeHash != InReferenceSet.ShaderCodeHash
		|| FeatureLevel != InReferenceSet.FeatureLevel)
	{
		return false;
	}

	if (ShaderTypeDependencies.Num() != InReferenceSet.ShaderTypeDependencies.Num())
	{
		return false;
	}

	if (LayoutParams != InReferenceSet.LayoutParams)
	{
		return false;
	}

	for (int32 ShaderIndex = 0; ShaderIndex < ShaderTypeDependencies.Num(); ShaderIndex++)
	{
		const FShaderTypeDependency& ShaderTypeDependency = ShaderTypeDependencies[ShaderIndex];

		if (ShaderTypeDependency != InReferenceSet.ShaderTypeDependencies[ShaderIndex])
		{
			return false;
		}
	}

	return true;
}

#if WITH_EDITOR

/**
 * Enqueues a compilation for a new shader of this type.
 * @param InKernel - The kernel to link the shader with.
 */
void FNNERuntimeIREEShaderType::BeginCompileShader(
	uint32 InShaderMapId,
	int32 PermutationId,
	const FNNERuntimeIREEResource* InKernel,
	FSharedShaderCompilerEnvironment* InCompilationEnvironment,
	EShaderPlatform InPlatform,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	FShaderTarget InTarget
	)
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(
		InShaderMapId, 
		FShaderCompileJobKey(this, InPlatform, nullptr, PermutationId),
		EShaderCompileJobPriority::Normal
		);

	TCHAR const* SourceFilePath = TEXT("/Plugin/NNERuntimeIREEShader/NNERuntimeIREEShader.usf");
	TCHAR const* GeneratedFilePath = TEXT("/Plugin/NNERuntimeIREEShader/Generated/NNERuntimeIREEShader.ush");

	NewJob->ShaderParameters = MakeShared<const FParameters, ESPMode::ThreadSafe>(*InKernel->GetShaderParamMetadata());
	NewJob->Input.SharedEnvironment = InCompilationEnvironment;
	NewJob->Input.Target = InTarget;
	NewJob->Input.ShaderFormat = LegacyShaderPlatformToShaderFormat(InPlatform);
	NewJob->Input.VirtualSourceFilePath = SourceFilePath;
	NewJob->Input.EntryPointName = InKernel->GetEntryPoint();
	NewJob->Input.Environment.IncludeVirtualPathToContentsMap.Add(GeneratedFilePath, InKernel->GetHLSLSource());
	UE_LOGF(LogNNERuntimeIREEShader, Verbose, "%ls", *InKernel->GetHLSLSource());
	
	AddUniformBufferIncludesToEnvironment(NewJob->Input.Environment, InPlatform);

	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	UE_LOGF(LogNNERuntimeIREEShader, Verbose, "			%ls", GetName());
	COOK_STAT(NNERuntimeIREEShaderCookStats::ShadersCompiled++);

	// InKernel->SetupCompileEnvironment(PermutationId, ShaderEnvironment);

	// Allow the shader type to modify the compile environment.
	SetupCompileEnvironment(InPlatform, InKernel, ShaderEnvironment);

	::GlobalBeginCompileShader(
		InKernel->GetFriendlyName(),
		nullptr,
		this,
		nullptr,//ShaderPipeline,
		PermutationId,
		SourceFilePath,
		*InKernel->GetEntryPoint(),
		FShaderTarget(GetFrequency(), InPlatform),
		NewJob->Input
		);

	NewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
}

/**
 * Either creates a new instance of this type or returns an equivalent existing shader.
 * @param InShaderMapHash - Precomputed hash of the shader map 
 * @param InCurrentJob - Compile job that was enqueued by BeginCompileShader.
 */
FShader* FNNERuntimeIREEShaderType::FinishCompileShader(
	uint64 InShaderMapHash,
	const FShaderCompileJob& InCurrentJob,
	const FString& InDebugDescription
	) const
{
	check(InCurrentJob.bSucceeded);

	FShader* Shader = ConstructCompiled(
		FNNERuntimeIREEShaderType::CompiledShaderInitializerType(
			this,
			static_cast<const FParameters*>(InCurrentJob.ShaderParameters.Get()),
			InCurrentJob.Key.PermutationId,
			InCurrentJob.Output,
			InShaderMapHash,
			InDebugDescription
			)
		);

	InCurrentJob.Output.ParameterMap.VerifyBindingsAreComplete(GetName(), InCurrentJob.Output.Target, InCurrentJob.Key.VFType);

	return Shader;
}
#endif // WITH_EDITOR

/**
 * Finds the shader map for a kernel.
 * @param InShaderMapId - The kernel id and static parameter set identifying the shader map
 * @param InPlatform - The platform to lookup for
 * @return nullptr if no cached shader map was found.
 */
FNNERuntimeIREEShaderMap* FNNERuntimeIREEShaderMap::FindId(const FNNERuntimeIREEShaderMapId& InShaderMapId, EShaderPlatform InPlatform)
{
	FNNERuntimeIREEShaderMap* Result = GIdToIREEShaderMap[InPlatform].FindRef(InShaderMapId);
	check(Result == nullptr || !Result->bDeletedThroughDeferredCleanup);
	return Result;
}

#if WITH_EDITOR

/**
* Compiles the shaders for a kernel and caches them in this shader map.
* @param InKernel - The kernel to compile shaders for.
* @param InShaderMapId - the kernel id and set of static parameters to compile for
* @param InPlatform - The platform to compile to
*/
void FNNERuntimeIREEShaderMap::Compile(
	FNNERuntimeIREEResource* InKernel, 
	const FNNERuntimeIREEShaderMapId& InShaderMapId,
	TRefCountPtr<FSharedShaderCompilerEnvironment> InCompilationEnvironment,
	const FNNERuntimeIREECompilationOutput& InKernelCompilationOutput,
	EShaderPlatform InPlatform,
	bool bSynchronousCompile,
	bool bApplyCompletedShaderMapForRendering
	)
{
	if (FPlatformProperties::RequiresCookedData())
	{
		UE_LOGF(LogNNERuntimeIREEShader, Fatal, "Trying to compile NNERuntimeIREE shader %ls at run-time, which is not supported on consoles!", *InKernel->GetFriendlyName() );
	}
	else
	{
		// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
		// Since it creates a temporary ref counted pointer.
		check(NumRefs > 0);
  
		// Add this shader map and to ShaderMapsBeingCompiled
		TArray<FNNERuntimeIREEResource*>* Kernel = ShaderMapsBeingCompiled.Find(this);
  
		if (Kernel)
		{
			check(!bSynchronousCompile);
			Kernel->AddUnique(InKernel);
		}
		else
		{
			// Assign a unique identifier so that shaders from this shader map can be associated with it after a deferred compile
			CompilingId = FShaderCommonCompileJob::GetNextJobId();
			UE_LOGF(LogNNERuntimeIREEShader, Verbose, "CompilingId = %p %d", InKernel, CompilingId);
			InKernel->AddCompileId(CompilingId);

			// Setup the compilation environment.
			// InKernel->SetupShaderCompilationEnvironment(InPlatform, *InCompilationEnvironment);
  
			// Store the kernel name for debugging purposes.
			FNNERuntimeIREEShaderMapContent* NewContent = new FNNERuntimeIREEShaderMapContent(InPlatform);
			NewContent->FriendlyName = InKernel->GetFriendlyName();
			NewContent->CompilationOutput = InKernelCompilationOutput;
			NewContent->ShaderMapId = InShaderMapId;
			AssignContent(NewContent);

			uint32 NumShaders = 0;
			TArray<FShaderCommonCompileJobPtr> NewJobs;
	
			// Iterate over all shader types.
			for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
			{
				FNNERuntimeIREEShaderType* ShaderType = ShaderTypeIt->GetNNERuntimeIREEShaderType();
				if (ShaderType && ShouldCacheNNERuntimeIREEShader(ShaderType, InPlatform, InKernel))
				{
					// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
					checkf(InShaderMapId.ContainsShaderType(ShaderType), TEXT("IREE kernel shader map %s missing expected shader type %s"), *GetFriendlyName(), ShaderType->GetName());
					
					// Compile this NNERuntimeIREE shader  
					for (int32 PermutationId = 0; PermutationId < InKernel->GetNumPermutations(); ++PermutationId)
					{
						// Only compile the shader if we don't already have it
						if (!NewContent->HasShader(ShaderType, PermutationId))
						{
							ShaderType->BeginCompileShader(
								CompilingId,
								PermutationId,
								InKernel,
								InCompilationEnvironment,
								InPlatform,
								NewJobs,
								FShaderTarget(ShaderType->GetFrequency(), GetShaderPlatform())
								);
						}
						NumShaders++;
					}
				}
				else if (ShaderType)
				{
					InKernel->RemoveOutstandingCompileId(CompilingId);
					const FString ShaderFormatName = FDataDrivenShaderPlatformInfo::GetShaderFormat(InPlatform).ToString();

					FString Message = FString::Printf(TEXT("%s: Compilation not supported on %s."), *InKernel->GetFriendlyName(), *ShaderFormatName);
					// InKernel->NotifyCompilationFinished(Message);
					UE_LOGF(LogNNERuntimeIREEShader, Fatal, "%ls", *Message);
				}
			}
  
			if (!Kernel)
			{
				UE_LOGF(LogNNERuntimeIREEShader, Verbose, "		%u Shaders", NumShaders);
			}

			// Register this shader map in the global NNERuntimeIREE->shadermap map
			Register(InPlatform);
  
			// Mark the shader map as not having been finalized with ProcessCompilationResults
			bCompilationFinalized = false;
  
			// Mark as not having been compiled
			bCompiledSuccessfully = false;
  
			if (NumShaders > 0)
			{
				GNNERuntimeIREEShaderCompilationManager.AddJobs(NewJobs);

				TArray<FNNERuntimeIREEResource*> NewCorrespondingKernels;
				NewCorrespondingKernels.Add(InKernel);
				ShaderMapsBeingCompiled.Add(this, NewCorrespondingKernels);
			}
  
			// Compile the shaders for this shader map now if not deferring and deferred compiles are not enabled globally
			if (bSynchronousCompile)
			{
				TArray<int32> CurrentShaderMapId;
				CurrentShaderMapId.Add(CompilingId);
				GNNERuntimeIREEShaderCompilationManager.FinishCompilation(*NewContent->FriendlyName, CurrentShaderMapId);
			}
		}
	}
}

FShader* FNNERuntimeIREEShaderMap::ProcessCompilationResultsForSingleJob(FShaderCompileJob& CurrentJob, uint64 InShaderMapHash)
{
	check(CurrentJob.Id == CompilingId);

	GetResourceCode()->AddShaderCompilerOutput(CurrentJob.Output, CurrentJob.Key, CurrentJob.Input.GenerateDebugInfo());

	FShader* Shader = nullptr;

	const FNNERuntimeIREEShaderType* NNERuntimeIREEShaderType = CurrentJob.Key.ShaderType->GetNNERuntimeIREEShaderType();
	check(NNERuntimeIREEShaderType);

	Shader = NNERuntimeIREEShaderType->FinishCompileShader(InShaderMapHash, CurrentJob, GetContent()->FriendlyName);

	bCompiledSuccessfully = CurrentJob.bSucceeded;

	FNNERuntimeIREEShader* NNERuntimeIREEShader = static_cast<FNNERuntimeIREEShader*>(Shader);
	check(NNERuntimeIREEShader);

	check(!GetContent()->HasShader(NNERuntimeIREEShaderType, CurrentJob.Key.PermutationId));
	return GetMutableContent()->FindOrAddShader(NNERuntimeIREEShaderType->GetHashedName(), CurrentJob.Key.PermutationId, Shader);
}

bool FNNERuntimeIREEShaderMap::ProcessCompilationResults(const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutJobIndex, float& InOutTimeBudget)
{
	check(InOutJobIndex < InCompilationResults.Num());

	double StartTime = FPlatformTime::Seconds();

	uint64 ShaderMapHash = GetContent()->ShaderMapId.GetKernelHash();

	do
	{
		ProcessCompilationResultsForSingleJob(static_cast<FShaderCompileJob&>(*InCompilationResults[InOutJobIndex].GetReference()), ShaderMapHash);

		InOutJobIndex++;
		
		double NewStartTime = FPlatformTime::Seconds();
		InOutTimeBudget -= NewStartTime - StartTime;
		StartTime = NewStartTime;
	}
	while ((InOutTimeBudget > 0.0f) && (InOutJobIndex < InCompilationResults.Num()));

	if (InOutJobIndex == InCompilationResults.Num())
	{
		FinalizeContent();

		// SaveToDerivedDataCache();
		// The shader map can now be used on the rendering thread
		bCompilationFinalized = true;
		return true;
	}

	return false;
}

#endif // WITH_EDITOR

bool FNNERuntimeIREEShaderMap::IsIREEShaderComplete(const FNNERuntimeIREEResource* InKernel, const FNNERuntimeIREEShaderType* InShaderType, bool bSilent)
{
	// If we should cache this kernel, it's incomplete if the shader is missing
	if (ShouldCacheNNERuntimeIREEShader(InShaderType, GetShaderPlatform(), InKernel))
	{
		for (int32 PermutationId = 0; PermutationId < InKernel->GetNumPermutations(); ++PermutationId)
		{
			if (!GetContent()->HasShader((FShaderType*)InShaderType, PermutationId))
			{
				if (!bSilent)
				{
					UE_LOGF(LogNNERuntimeIREEShader, Warning, "Incomplete shader %ls, missing FNNERuntimeIREEShader %ls.", *InKernel->GetFriendlyName(), InShaderType->GetName());
				}
				return false;
			}
		}
	}

	return true;
}

bool FNNERuntimeIREEShaderMap::IsComplete(const FNNERuntimeIREEResource* InKernel, bool bSilent)
{
	// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
	// Since it creates a temporary ref counted pointer.
	check(NumRefs > 0);

#if WITH_EDITOR
	const TArray<FNNERuntimeIREEResource*>* CorrespondingKernels = FNNERuntimeIREEShaderMap::ShaderMapsBeingCompiled.Find(this);
	if (CorrespondingKernels)
	{
		check(!bCompilationFinalized);
		return false;
	}
#endif

	// Iterate over all shader types.
	for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		// Find this shader type in the kernel's shader map.
		const FNNERuntimeIREEShaderType* ShaderType = ShaderTypeIt->GetNNERuntimeIREEShaderType();
		if (ShaderType && !IsIREEShaderComplete(InKernel, ShaderType, bSilent))
		{
			return false;
		}
	}

	return true;
}

void FNNERuntimeIREEShaderMap::GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, 0u, OutShaders);
}

void FNNERuntimeIREEShaderMap::GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, OutShaders);
}

void FNNERuntimeIREEShaderMap::GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const
{
	GetContent()->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
}

/**
 * Registers a NNERuntimeIREE shader map in the global map.
 */
void FNNERuntimeIREEShaderMap::Register(EShaderPlatform InShaderPlatform)
{
	if (!bRegistered)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShaderMaps);
	}

	GIdToIREEShaderMap[GetShaderPlatform()].Add(GetContent()->ShaderMapId,this);
	bRegistered = true;
}

void FNNERuntimeIREEShaderMap::AddRef()
{
	check(!bDeletedThroughDeferredCleanup);
	FPlatformAtomics::InterlockedIncrement(&NumRefs);
}

void FNNERuntimeIREEShaderMap::Release()
{
	check(NumRefs > 0);
	if(FPlatformAtomics::InterlockedDecrement(&NumRefs) == 0)
	{
		if (bRegistered)
		{
			DEC_DWORD_STAT(STAT_Shaders_NumShaderMaps);

			GIdToIREEShaderMap[GetShaderPlatform()].Remove(GetContent()->ShaderMapId);
			bRegistered = false;
		}

		check(!bDeletedThroughDeferredCleanup);
		bDeletedThroughDeferredCleanup = true;
		BeginCleanup(this);
	}
}

FNNERuntimeIREEShaderMap::FNNERuntimeIREEShaderMap() :
	CompilingId(1),
	NumRefs(0),
	bDeletedThroughDeferredCleanup(false),
	bRegistered(false),
	bCompilationFinalized(true),
	bCompiledSuccessfully(true)/*,
	bIsPersistent(true) */
{
	// checkSlow(IsInGameThread() || IsAsyncLoading());
	AllKernelShaderMaps.Add(this);
}

FNNERuntimeIREEShaderMap::~FNNERuntimeIREEShaderMap()
{ 
	checkSlow(IsInGameThread() || IsAsyncLoading());
	check(bDeletedThroughDeferredCleanup);
	check(!bRegistered);
	AllKernelShaderMaps.RemoveSwap(this);
}

bool FNNERuntimeIREEShaderMap::Serialize(FArchive& Ar)
{
	FShaderSerializeContext Ctx(Ar);
	return Super::Serialize(Ctx);
}

#endif // WITH_NNE_RUNTIME_IREE_SHADER