// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEShaderMetadataAllocations.h"

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

FNNERuntimeIREEShaderParametersMetadataAllocations::~FNNERuntimeIREEShaderParametersMetadataAllocations()
{
	
}

#endif // WITH_NNE_RUNTIME_IREE_SHADER