// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEShaderModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "NNERuntimeIREEShaderLog.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogNNERuntimeIREEShader);

void FNNERuntimeIREEShaderModule::StartupModule()
{
#ifdef WITH_NNE_RUNTIME_IREE_SHADER
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("NNERuntimeIREE"));
	if (Plugin.IsValid())
	{
		const FString ShadersDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Shaders"));

		AddShaderSourceDirectoryMapping(TEXT("/Plugin/NNERuntimeIREEShader"), ShadersDir);
	}
	else
	{
		UE_LOGF(LogNNERuntimeIREEShader, Warning, "Shaders directory not added. Failed to find NNERuntimeIREE plugin");
	}
#endif // WITH_NNE_RUNTIME_IREE_SHADER
}

void FNNERuntimeIREEShaderModule::ShutdownModule()
{

}

IMPLEMENT_MODULE(FNNERuntimeIREEShaderModule, NNERuntimeIREEShader)