// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeIREEShaderShared.h"

#ifdef WITH_NNE_RUNTIME_IREE_SHADER

#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatform.h"
#include "NNERuntimeIREEShader.h"
#include "NNERuntimeIREEShaderCompilationManager.h"
#include "NNERuntimeIREEShaderLog.h"
#include "NNERuntimeIREEShaderMetadataAllocations.h"
#include "RendererInterface.h"
#include "RHIShaderFormatDefinitions.inl"
#include "ShaderCompiler.h"
#include "ShaderParameterMetadataBuilder.h"
#include "Stats/StatsMisc.h"
#include "TextureResource.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

IMPLEMENT_TYPE_LAYOUT(FNNERuntimeIREECompilationOutput);
IMPLEMENT_TYPE_LAYOUT(FNNERuntimeIREEShaderMapId);
IMPLEMENT_TYPE_LAYOUT(FNNERuntimeIREEShaderMapContent);

FNNERuntimeIREEResource::FNNERuntimeIREEResource() : bLoadedCookedShaderMapId(false)
{
}

FNNERuntimeIREEResource::~FNNERuntimeIREEResource()
{

}

bool FNNERuntimeIREEResource::ShouldCache(EShaderPlatform InPlatform, const FShaderType* InShaderType) const
{
	check(InShaderType->GetNNERuntimeIREEShaderType())
	return true;
}

bool FNNERuntimeIREEResource::SerializeShaderMap(FArchive& Ar)
{
	bool bSuccess = false;

	if (Ar.IsSaving())
	{
#if WITH_EDITOR
		FinishCompilation();

		bool bValid = bSuccess = GameThreadShaderMap != nullptr && GameThreadShaderMap->CompiledSuccessfully();
		Ar << bValid;

		if (bValid)
		{
			GameThreadShaderMap->AssociateWithAsset(AssetPath);
			GameThreadShaderMap->Serialize(Ar);
		}
#endif
	}
	else
	{
		bool bValid = false;
		Ar << bValid;

		if (bValid)
		{
			TRefCountPtr<FNNERuntimeIREEShaderMap> LoadedShaderMap = new FNNERuntimeIREEShaderMap();
			bSuccess = LoadedShaderMap->Serialize(Ar);

			// Toss the loaded shader data if this is a server only instance or if it's for a different RHI than the current one.
			// todo[CF] Don't cook it in the first place
			if (bSuccess && FApp::CanEverRender())
			{
				GameThreadShaderMap = RenderingThreadShaderMap = LoadedShaderMap;
				GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
			}
		}
	}

	return bSuccess;
}

void FNNERuntimeIREEResource::SetRenderingThreadShaderMap(FNNERuntimeIREEShaderMap* InShaderMap)
{
	check(IsInRenderingThread());
	RenderingThreadShaderMap = InShaderMap;
}

void FNNERuntimeIREEResource::RemoveOutstandingCompileId(const int32 InOldOutstandingCompileShaderMapId)
{
	if (0 <= OutstandingCompileShaderMapIds.Remove(InOldOutstandingCompileShaderMapId))
	{
		UE_LOGF(LogNNERuntimeIREEShader, Verbose, "RemoveOutstandingCompileId %p %d", this, InOldOutstandingCompileShaderMapId);
	}
}

void FNNERuntimeIREEResource::NotifyCompilationFinished(FString const& ResultMessage)
{
	UE_LOGF(LogNNERuntimeIREEShader, Log, "%ls", *ResultMessage);
	FNNERuntimeIREEShaderCompileMessage Message;
	Message.Type = FNNERuntimeIREEShaderCompileMessage::EMessageType::Info;
	Message.Text = ResultMessage;
	CompilationResults.Messages.Add(Message);
	// OnCompilationCompleteDelegate.ExecuteIfBound(this);
}

#if WITH_EDITOR
void FNNERuntimeIREEResource::FinishCompilation()
{
	TArray<int32> ShaderMapIdsToFinish;
	GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);

	if (ShaderMapIdsToFinish.Num() > 0)
	{
		for (int32 i = 0; i < ShaderMapIdsToFinish.Num(); i++)
		{
			UE_LOGF(LogNNERuntimeIREEShader, Verbose, "FinishCompilation()[%d] %ls id %d!", i, *GetFriendlyName(), ShaderMapIdsToFinish[i]);
		}
	
		// Block until the shader maps that we will save have finished being compiled
		GNNERuntimeIREEShaderCompilationManager.FinishCompilation(*GetFriendlyName(), ShaderMapIdsToFinish);

		// Shouldn't have anything left to do...
		TArray<int32> ShaderMapIdsToFinish2;
		GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish2);
		ensure(ShaderMapIdsToFinish2.Num() == 0);
	}
}
#endif // WITH_EDITOR

void FNNERuntimeIREEResource::GetDependentShaderTypes(EShaderPlatform InPlatform, TArray<FShaderType*>& OutShaderTypes) const
{
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FNNERuntimeIREEShaderType* ShaderType = ShaderTypeIt->GetNNERuntimeIREEShaderType();

		if (ShaderType && ShaderType->ShouldCache(InPlatform, this) && ShouldCache(InPlatform, ShaderType) )
		{
			OutShaderTypes.Add(ShaderType);
		}
	}

	OutShaderTypes.Sort(FCompareShaderTypes());
}

void FNNERuntimeIREEResource::GetShaderMapId(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, FNNERuntimeIREEShaderMapId& OutId) const
{
	if (bLoadedCookedShaderMapId)
	{
		OutId = CookedShaderMapId;
	}
	else
	{
		TArray<FShaderType*> ShaderTypes;
		GetDependentShaderTypes(InPlatform, ShaderTypes);

		OutId.FeatureLevel = GetFeatureLevel();
		OutId.ShaderCodeHash = ShaderCodeHash;
#if WITH_EDITOR
		OutId.SetShaderDependencies(ShaderTypes, InPlatform);
		if (TargetPlatform)
		{
			OutId.LayoutParams.InitializeForPlatform(TargetPlatform->IniPlatformName(), TargetPlatform->HasEditorOnlyData());
		}
		else
		{
			OutId.LayoutParams.InitializeForCurrent();
		}
#else
		if (TargetPlatform != nullptr)
		{
			UE_LOGF(LogNNERuntimeIREEShader, Error, "FNNERuntimeIREEResource::GetShaderMapId: TargetPlatform is not null, but a cooked executable cannot target platforms other than its own.");
		}
		OutId.LayoutParams.InitializeForCurrent();
#endif
	}
}

bool FNNERuntimeIREEResource::CacheShaders(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	FNNERuntimeIREEShaderMapId ShaderMapId;
	GetShaderMapId(InPlatform, TargetPlatform, ShaderMapId);
	return CacheShaders(ShaderMapId, InPlatform, bApplyCompletedShaderMapForRendering, bSynchronous);
}

bool FNNERuntimeIREEResource::CacheShaders(const FNNERuntimeIREEShaderMapId& InShaderMapId, EShaderPlatform InPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	// Find the kernel's cached shader map.
	GameThreadShaderMap = FNNERuntimeIREEShaderMap::FindId(InShaderMapId, InPlatform);
	if (GameThreadShaderMap && GameThreadShaderMap->IsComplete(this, false))
	{
		return true;
	}

	bool bSucceeded = false;

#if WITH_EDITOR
	// If there's no cached shader map for this kernel compile a new one.
	// This is just kicking off the compile, GameThreadShaderMap will not be complete yet
	bSucceeded = BeginCompileShaderMap(InShaderMapId, InPlatform, GameThreadShaderMap, bApplyCompletedShaderMapForRendering, bSynchronous);
#endif // WITH_EDITOR

	if (!bSucceeded)
	{
		GameThreadShaderMap = nullptr;
	}

	return bSucceeded;
}

void FNNERuntimeIREEResource::SetupResource(
	ERHIFeatureLevel::Type InFeatureLevel,
	FString const& InFriendlyName,
	FString const& InShaderEntryPoint,
	FString const& InShaderHashKey,
	FString const& InShaderSource,
	TUniquePtr<FNNERuntimeIREEShaderParametersMetadataAllocations> InShaderParameterMetadataAllocations,
	FShaderParametersMetadata* InShaderParameterMetadata,
	FName const& InAssetPath,
	TConstArrayView<uint32> InBufferBindings
	)
{
	FeatureLevel = InFeatureLevel;
	FriendlyName = InFriendlyName;
	ShaderEntryPoint = InShaderEntryPoint;
	ShaderCodeHash = GetTypeHash(InShaderHashKey);
	ShaderSource = InShaderSource;
	ShaderParameterMetadataAllocations = MoveTemp(InShaderParameterMetadataAllocations);
	ShaderParameterMetadata = InShaderParameterMetadata;
	CompilationResults.Messages.Reset();
	AssetPath = InAssetPath;
	BufferBindings = InBufferBindings;
}

TShaderRef<FNNERuntimeIREEShader> FNNERuntimeIREEResource::GetShader(int32 PermutationId) const
{
	check(!GIsThreadedRendering || !IsInGameThread());
	if (!GIsEditor || RenderingThreadShaderMap)
	{
		return RenderingThreadShaderMap->GetShader<FNNERuntimeIREEShader>(PermutationId);
	}
	return TShaderRef<FNNERuntimeIREEShader>();
}

bool FNNERuntimeIREEResource::IsSame(const FNNERuntimeIREEShaderMapId& InIdentifier) const
{
	return
		InIdentifier.ShaderCodeHash == ShaderCodeHash &&
		InIdentifier.FeatureLevel == FeatureLevel;
}

uint32 FNNERuntimeIREEResource::GetBindingIndex(uint32 BufferIdx) const
{
	check(BufferIdx < (uint32)BufferBindings.Num());
	return BufferBindings[BufferIdx];
}

#if WITH_EDITOR
void FNNERuntimeIREEResource::GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& OutShaderMapIds)
{
	// Build an array of the shader map Id's are not finished compiling.
	if (GameThreadShaderMap && GameThreadShaderMap.IsValid() && !GameThreadShaderMap->IsCompilationFinalized())
	{
		OutShaderMapIds.Add(GameThreadShaderMap->GetCompilingId());
	}
	else if (OutstandingCompileShaderMapIds.Num() != 0)
	{
		OutShaderMapIds.Append(OutstandingCompileShaderMapIds);
	}
}

/**
 * Compiles this kernel for Platform, storing the result in OutShaderMap
 *
 * @param InShaderMapId - the set of static parameters to compile
 * @param InPlatform - the platform to compile for
 * @param OutShaderMap - the shader map to compile
 * @return - true if compile succeeded or was not necessary (shader map for ShaderMapId was found and was complete)
 */
bool FNNERuntimeIREEResource::BeginCompileShaderMap(const FNNERuntimeIREEShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FNNERuntimeIREEShaderMap>& OutShaderMap, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	bool bSuccess = false;

	STAT(double NNERuntimeIREEShaderCompileTime = 0);

	SCOPE_SECONDS_COUNTER(NNERuntimeIREEShaderCompileTime);

	TRefCountPtr<FNNERuntimeIREEShaderMap> NewShaderMap = new FNNERuntimeIREEShaderMap();

	// Create a shader compiler environment for the material that will be shared by all jobs from this material
	TRefCountPtr<FSharedShaderCompilerEnvironment> Environment = new FSharedShaderCompilerEnvironment();

	// Compile the shaders for the kernel.
	FNNERuntimeIREECompilationOutput CompilationOutput;
	NewShaderMap->Compile(this, InShaderMapId, Environment, CompilationOutput, InPlatform, bSynchronous, bApplyCompletedShaderMapForRendering);

	if (bSynchronous)
	{
		// If this is a synchronous compile, assign the compile result to the output
		OutShaderMap = NewShaderMap->CompiledSuccessfully() ? NewShaderMap : nullptr;
	}
	else
	{
		UE_LOGF(LogNNERuntimeIREEShader, Verbose, "BeginCompileShaderMap AddUnique %p %d", this, NewShaderMap->GetCompilingId());
		unimplemented();
		// OutstandingCompileShaderMapIds.AddUnique(NewShaderMap->GetCompilingId());
		
		// Async compile, use nullptr to detect it if used
		OutShaderMap = nullptr;
	}

	return true;
}

void FNNERuntimeIREEShaderMapId::SetShaderDependencies(const TArray<FShaderType*>& InShaderTypes, EShaderPlatform InShaderPlatform)
{
	if (!FPlatformProperties::RequiresCookedData())
	{
		for (FShaderType* ShaderType : InShaderTypes)
		{
			if (ShaderType != nullptr)
			{
				FShaderTypeDependency Dependency;
				Dependency.ShaderTypeName = ShaderType->GetHashedName();
				Dependency.SourceHash = ShaderType->GetSourceHash(InShaderPlatform);
				ShaderTypeDependencies.Add(Dependency);
			}
		}
	}
}

#endif // WITH_EDITOR

bool FNNERuntimeIREEShaderMapId::ContainsShaderType(const FShaderType* ShaderType) const
{
	for (int32 TypeIndex = 0; TypeIndex < ShaderTypeDependencies.Num(); TypeIndex++)
	{
		if (ShaderTypeDependencies[TypeIndex].ShaderTypeName == ShaderType->GetHashedName())
		{
			return true;
		}
	}

	return false;
}

#endif // WITH_NNE_RUNTIME_IREE_SHADER